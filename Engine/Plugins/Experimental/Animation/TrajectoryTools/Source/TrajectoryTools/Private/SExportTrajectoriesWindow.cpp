// Copyright Epic Games, Inc. All Rights Reserved.

#include "SExportTrajectoriesWindow.h"

#include "Common/ProviderLock.h"
#include "PropertyCustomizationHelpers.h"
#include "SSimpleTimeSlider.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Layout/WidgetPath.h"
#include "Widgets/SBoxPanel.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"
#include "Animation/AnimSequence.h"
#include "Viewports.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Font.h"
#include "CanvasTypes.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SceneView.h"
#include "SWarningOrErrorBox.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/Skeleton.h"
#include "TrajectoryExportOperation.h"
#include "Fonts/FontMeasure.h"
#include "Widgets/Layout/SBorder.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SExportTrajectoriesWindow)

#define LOCTEXT_NAMESPACE "SExportTrajectoriesWindow"

int32 GetSampleIndexFromTime(const FFrameRate& InFrameRate, double InTime, int32 InSampleNum)
{
	return FMath::Clamp(InFrameRate.AsFrameTime(InTime).FloorToFrame().Value, 0, InSampleNum - 1);
}

void PreviewViewportFocusOnTrajectory(const TSharedPtr<FPreviewData>& InPreviewData, const TSharedPtr<FEditorViewportClient>& InViewportClient)
{
	if (!InPreviewData || !InViewportClient)
	{
		return;
	}

	// Build trajectory bounds
	FBoxSphereBounds::Builder BoundsBuilder;
	for (const FGameplayTrajectory::FSample& OutputSample : InPreviewData->OutputTrajectory.Samples)
    {
		BoundsBuilder += OutputSample.Position;
    }
	
	const FBoxSphereBounds Bound = BoundsBuilder.IsValid() ? BoundsBuilder : FBoxSphereBounds(FSphere(FVector::ZeroVector, 250.0f));

	// Focus
	InViewportClient->SetViewLocation(Bound.Origin);
	InViewportClient->SetViewLocationForOrbiting(Bound.Origin, Bound.SphereRadius * 1.5f);
}

void SPreviewTrajectoryViewport::Construct(const FArguments& InArgs)
{
	SEditorViewport::Construct(SEditorViewport::FArguments()
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		.AddMetaData<FTagMetaData>(TEXT("TrajectoryExport.Viewport")));

	PreviewData = InArgs._PreviewData;

	PreviewComponent = NewObject<UDebugSkelMeshComponent>();

	// Always refresh pose and ignore root motion (as position is dictated by trajectory)
	PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	PreviewComponent->SetProcessRootMotionMode(EProcessRootMotionMode::Ignore);

	// No need to tick the preview component as we will me manually force it to.
	PreviewComponent->PrimaryComponentTick.bStartWithTickEnabled = false;
	PreviewComponent->PrimaryComponentTick.bCanEverTick = false;

	// No anim instance.
	PreviewComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);

	// Ensure preview component gets ticked.
	PreviewScene.AddComponent(PreviewComponent, FTransform::Identity);
}

FReply SPreviewTrajectoryViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::F)
	{
		PreviewViewportFocusOnTrajectory(PreviewData, GetViewportClient());
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SPreviewTrajectoryViewport::ResetPreviewSkeletalMesh() const
{
	// Exit on invalid state.
	const bool bHasNoSelectedTrajectory = PreviewData->SourceIndex == INDEX_NONE;
	if (bHasNoSelectedTrajectory || !IRewindDebugger::Instance() || !IRewindDebugger::Instance()->GetAnalysisSession())
	{
		return;
	}

	// Get data providers for traced objets.
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();
	const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
	const IAnimationProvider* AnimationProvider = Session->ReadProvider<IAnimationProvider>("AnimationProvider");

	// Preview traced skeletal mesh.
	if (GameplayProvider && AnimationProvider)
	{
		FSkeletalMeshInfo SkeletalMeshInfoToPreview {};
		{
			const int32 ScrubSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Timeline.ScrubTime, PreviewData->OutputTrajectory.Samples.Num());

			for (int i = 0; i < PreviewData->OutputTrajectory.TraceInfo.Num(); ++i)
			{
				if (PreviewData->OutputTrajectory.TraceInfo.Ranges[i].Contains(ScrubSampleIndex))
				{
					SkeletalMeshInfoToPreview = PreviewData->OutputTrajectory.TraceInfo.SkeletalMeshInfos[i];
					break;
				}
			}
		}
		ensureMsgf(SkeletalMeshInfoToPreview.Id != 0, TEXT("Invalid skeletal mesh to preview. If this gets hit something went wrong."));

		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			if (const FSkeletalMeshInfo* SkeletalMeshInfo = AnimationProvider->FindSkeletalMeshInfo(SkeletalMeshInfoToPreview.Id /*PreviewData->Viewport.SourceSkeletalMeshInfo.Id*/))
			{
				TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
				if (const FObjectInfo* SkeletalMeshObjectInfo = GameplayProvider->FindObjectInfo(SkeletalMeshInfoToPreview.Id /*PreviewData->Viewport.SourceSkeletalMeshInfo.Id)*/))
				{
					USkeletalMesh* SkeletalMesh = TSoftObjectPtr<USkeletalMesh>(FSoftObjectPath(SkeletalMeshObjectInfo->PathName)).LoadSynchronous();

					// Try the skeleton asset preview mesh as a fallback when skeletal mesh asset is not valid.
					if (SkeletalMesh == nullptr)
					{
						// Ensure our skeleton is valid.
						if (SkeletalMeshInfo->SkeletonId != 0)
						{
							const FObjectInfo& SkeletonObjectInfo = GameplayProvider->GetObjectInfo(SkeletalMeshInfo->SkeletonId);
					
							USkeleton* Skeleton = TSoftObjectPtr<USkeleton>(FSoftObjectPath(SkeletonObjectInfo.PathName)).LoadSynchronous();
							if (Skeleton)
							{
								SkeletalMesh = Skeleton->GetPreviewMesh(true);
							}
						}
					}

					// Assign loaded skeletal mesh.
					if (SkeletalMesh)
					{
						PreviewComponent->SetSkeletalMesh(SkeletalMesh);
					}
				}
			}
		}

		// Run method that sets world position and bone transforms.
		UpdatePreviewPoseFromScrubTime(0, RewindDebugger->CurrentTraceTime());
	}
}

void SPreviewTrajectoryViewport::UpdatePreviewPoseFromScrubTime(double InTraceStartTime, double InTraceEndTime) const
{
	if (PreviewComponent->GetSkeletalMeshAsset() == nullptr || PreviewData->SourceIndex == INDEX_NONE)
	{
		return;
	}

	if (PreviewData->SourceTrajectories[PreviewData->SourceIndex].Samples.IsEmpty() || PreviewData->SourceTrajectories[PreviewData->SourceIndex].Poses.IsEmpty())
	{
		return;
	}
	
	const int32 ScrubSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Timeline.ScrubTime, PreviewData->OutputTrajectory.Samples.Num());
	const double ScrubSampleTime = PreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Time;
	const FVector ScrubSamplePosition = PreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Position;
	const FQuat ScrubSampleOrientation = PreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Orientation;

	// Update transform first, in case, we aren't able to update the pose itself.
	PreviewComponent->SetWorldTransform(FTransform(ScrubSampleOrientation, ScrubSamplePosition), false, nullptr, ETeleportType::TeleportPhysics);

	// Get preview pose buffer.
	TArray<FTransform>& EditablePose = PreviewComponent->GetEditableComponentSpaceTransforms();
	if (EditablePose.Num() == PreviewData->OutputTrajectory.Poses[ScrubSampleIndex].Num())
	{
		EditablePose = PreviewData->OutputTrajectory.Poses[ScrubSampleIndex];
	}
	else
	{
		UE_LOGFMT(LogTemp, Warning, "Preview Skeletal Mesh Cmp transform buffer does not match the recorded sample pose buffer size. {num1} vs {num2}", EditablePose.Num(), PreviewData->OutputTrajectory.Poses[ScrubSampleIndex].Num());
		return;
	}

	// Update current preview pose.
	FTrajectoryToolsLibrary::GetPoseAtTimeInTrajectory(PreviewData->SourceTrajectories[PreviewData->SourceIndex], ScrubSampleTime, EditablePose);
	
	// Apply new transforms.
	PreviewComponent->ApplyEditedComponentSpaceTransforms();

	// @todo: Handle LOD setting.
	// PreviewComponent->SetForcedLOD(PoseMessage->LodIndex + 1);

	// Update preview component's properties.
	PreviewComponent->UpdateLODStatus();
	PreviewComponent->UpdateChildTransforms(EUpdateTransformFlags::None, ETeleportType::TeleportPhysics);
	PreviewComponent->SetVisibility(true);
	PreviewComponent->MarkRenderStateDirty();
	PreviewComponent->SetDrawDebugSkeleton(true);
}

TSharedRef<FEditorViewportClient> SPreviewTrajectoryViewport::MakeEditorViewportClient()
{
	TSharedPtr<FEditorViewportClient> EditorViewportClient = MakeShareable(new FPreviewTrajectoryViewportClient(PreviewScene, SharedThis(this)));
	return EditorViewportClient.ToSharedRef();
}

FPreviewTrajectoryViewportClient::FPreviewTrajectoryViewportClient(FPreviewScene& InPreviewScene, const TSharedRef<SPreviewTrajectoryViewport>& InPreviewTrajectoryViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InPreviewTrajectoryViewport))
{
	ViewportType = LVT_Perspective;
	bSetListenerPosition = false;

	// Use defaults for view transforms.
	SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);
	SetViewLocationForOrbiting(FVector::ZeroVector);

	// Normally the bIsRealtime flag is determined by whether the connection is remote, but our
	// tools require always being ticked.
	SetRealtime(true);

	// Lit gives us the most options in terms of the materials we can use.
	SetViewMode(VMI_Lit);
	
	// Allow for camera control.
	bUsingOrbitCamera = true;
	bDisableInput = false;
	
	// This seems to be needed to get the correct world time in the preview.
	// SetIsSimulateInEditorViewport(true);
	
	// Always composite editor objects after post-processing in the editor
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	EngineShowFlags.DisableAdvancedFeatures();
	
	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(40, 40, 40);
	DrawHelper.GridColorMajor = FColor(20, 20, 20);
	DrawHelper.GridColorMinor =  FColor(10, 10, 10);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
}

void FPreviewTrajectoryViewportClient::Tick(float DeltaTime)
{
	FEditorViewportClient::Tick(DeltaTime);
	
	// Used to update preview pose.
	TWeakPtr<SPreviewTrajectoryViewport> PreviewEditorViewport = StaticCastWeakPtr<SPreviewTrajectoryViewport>(EditorViewportWidget);

	if (PreviewEditorViewport.IsValid())
	{
		if (IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
		{
			// Get data providers for traced objets.
			if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
			{
				PreviewEditorViewport.Pin().Get()->UpdatePreviewPoseFromScrubTime(0, RewindDebugger->CurrentTraceTime());
			}
		}
	}

	// Tick preview components.
	if (PreviewScene)
	{
		PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaTime);
	}
}

void FPreviewTrajectoryViewportClient::Draw(const FSceneView* InView, FPrimitiveDrawInterface* InPDI)
{
	constexpr float DepthBias = 4.0f;
	constexpr bool bScreenSpace = true;
	
	FEditorViewportClient::Draw(InView, InPDI);

	if (const TSharedPtr<SPreviewTrajectoryViewport> ViewportWidget = StaticCastSharedPtr<SPreviewTrajectoryViewport>(EditorViewportWidget.Pin()))
	{
		if (const TSharedPtr<FPreviewData> PreviewData = ViewportWidget->GetPreviewData())
		{
			if (!PreviewData->OutputTrajectory.Samples.IsEmpty())
			{
				// Are we exporting a segment of the trajectory
				const FFloatInterval& ExportRange = PreviewData->Details->ExportSettings.Range;
				const bool bShouldDrawEntireTrajectory = FMath::IsNearlyZero(ExportRange.Size()) /*|| !ExportRange.IsValid()*/;

				// Draw trajectory
				for (int SampleIndex = 0; SampleIndex < PreviewData->OutputTrajectory.Samples.Num(); ++SampleIndex)
				{
					// Current sample info
					const FGameplayTrajectory::FSample& Sample = PreviewData->OutputTrajectory.Samples[SampleIndex];
					const bool bIsSampleInRange = ExportRange.Contains(Sample.Time);
					FTransform SampleTransform(Sample.Orientation, Sample.Position);
					const uint8 SampleAlpha = bShouldDrawEntireTrajectory ? 255 : (bIsSampleInRange ? 255 : 25);
					FLinearColor SampleColor = FColor::Black.WithAlpha(SampleAlpha);
					
					// Draw vertical ticks indicating start and end locations.
					constexpr double SampleTickSize = 5.0;
					InPDI->DrawTranslucentLine(Sample.Position, Sample.Position + SampleTransform.GetUnitAxis(EAxis::Z) * SampleTickSize, SampleColor, SDPG_World, 1.0f, DepthBias, bScreenSpace);

					// Draw line connecting current sample to next sample
					if (SampleIndex + 1 < PreviewData->OutputTrajectory.Samples.Num())
					{
						const FGameplayTrajectory::FSample& FutureSample = PreviewData->OutputTrajectory.Samples[SampleIndex + 1];
						const bool bIsFutureSampleInRange = ExportRange.Contains(FutureSample.Time);
						const uint8 FutureSampleAlpha = bIsSampleInRange && bIsFutureSampleInRange ? 255 : 25;
						FLinearColor FutureSampleColor = FColor::Black.WithAlpha(FutureSampleAlpha);

						InPDI->DrawTranslucentLine(Sample.Position, FutureSample.Position, FutureSampleColor, SDPG_World, 1.5f, DepthBias, bScreenSpace);
					}
				}

				// Draw scrubber point
				if (PreviewData->Timeline.bShouldDrawScrubber)
				{
					const int32 ScrubSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Timeline.ScrubTime, PreviewData->OutputTrajectory.Samples.Num());
					
					InPDI->DrawPoint(PreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Position, FColor::White, 7.0f, SDPG_Foreground);
				}

				// Draw start and end export range points
				if (PreviewData->Details->ExportSettings.bShouldForceOrigin)
				{
					const int32 OriginSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.OriginTime,PreviewData->OutputTrajectory.Samples.Num());
					
					InPDI->DrawPoint(PreviewData->OutputTrajectory.Samples[OriginSampleIndex].Position, FColor::Blue, 7.0f, SDPG_Foreground);
				}
			}
		}
	}
}

void FPreviewTrajectoryViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& InView, FCanvas& InCanvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, InView, InCanvas);
	
	if (const TSharedPtr<SPreviewTrajectoryViewport> ViewportWidget = StaticCastSharedPtr<SPreviewTrajectoryViewport>(EditorViewportWidget.Pin()))
	{
		if (const TSharedPtr<FPreviewData> PreviewData = ViewportWidget->GetPreviewData())
		{
			if (!PreviewData->OutputTrajectory.Samples.IsEmpty())
			{
				const auto DrawInfoStringForSample = [](FSceneView& InView, FCanvas& InCanvas, uint32 InSampleIndex, double InSampleTime, const FVector& InSamplePosition)
				{
					FVector2D PixelLocation;
					if (InView.WorldToPixel(InSamplePosition,PixelLocation))
					{
						PixelLocation.X = FMath::RoundToFloat(PixelLocation.X);
						PixelLocation.Y = FMath::RoundToFloat(PixelLocation.Y);

						constexpr FColor LabelColor(200, 200, 200);
						constexpr FLinearColor ShadowColor(0, 0, 0, 0.3f);
						const UFont* Font = GEngine->GetLargeFont();

						PixelLocation.Y -= Font->GetMaxCharHeight() * 2;
						InCanvas.DrawShadowedString(PixelLocation.X, PixelLocation.Y, FString::Format(TEXT("{0}* ({1}s)"), {InSampleIndex, InSampleTime}), Font, LabelColor, ShadowColor);
					}
				};

				// Draw scrubber point
				if (PreviewData->Timeline.bShouldDrawScrubber)
				{
					const int32 ScrubSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Timeline.ScrubTime, PreviewData->OutputTrajectory.Samples.Num());
					const double ScrubSampleTime = PreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Time;
					const FVector& ScrubSamplePosition = PreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Position;

					DrawInfoStringForSample(InView, InCanvas, ScrubSampleIndex, ScrubSampleTime,  ScrubSamplePosition);
				}

				// Draw origin point
				if (PreviewData->Details->ExportSettings.bShouldForceOrigin)
				{
					const int32 OriginSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.OriginTime,PreviewData->OutputTrajectory.Samples.Num());
					const double OriginSampleTime = PreviewData->OutputTrajectory.Samples[OriginSampleIndex].Time;
					const FVector& OriginSamplePosition = PreviewData->OutputTrajectory.Samples[OriginSampleIndex].Position;
					
					DrawInfoStringForSample(InView, InCanvas, OriginSampleIndex, OriginSampleTime,  OriginSamplePosition);
				}
				
				// Draw start and end export range points
				const FFloatInterval& ExportRange = PreviewData->Details->ExportSettings.Range;
				const bool bDisplayBoundInfo = !FMath::IsNearlyZero(ExportRange.Size()) && ExportRange.IsValid();
				if (bDisplayBoundInfo)
				{
					const int32 LowerBoundSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.Range.Min, PreviewData->OutputTrajectory.Samples.Num());
					const double LowerBoundSampleTime = PreviewData->OutputTrajectory.Samples[LowerBoundSampleIndex].Time;
					const FVector& LowerBoundSamplePosition = PreviewData->OutputTrajectory.Samples[LowerBoundSampleIndex].Position;
					
					const int32 UpperBoundSampleIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.Range.Max, PreviewData->OutputTrajectory.Samples.Num());
					const double UpperBoundSampleTime = PreviewData->OutputTrajectory.Samples[UpperBoundSampleIndex].Time;
					const FVector& UpperBoundSamplePosition = PreviewData->OutputTrajectory.Samples[UpperBoundSampleIndex].Position;

					DrawInfoStringForSample(InView, InCanvas, LowerBoundSampleIndex, LowerBoundSampleTime,  LowerBoundSamplePosition);
					DrawInfoStringForSample(InView, InCanvas, UpperBoundSampleIndex, UpperBoundSampleTime,  UpperBoundSamplePosition);
				}
			}
		}
	}
}

/** Used to preview a point in time of a given trajectory */
class STrajectoryTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STrajectoryTimeline) {}
		SLATE_ARGUMENT(TSharedPtr<FPreviewData>, PreviewData)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		SLATE_EVENT(SSimpleTimeSlider::FOnScrubPositionChanged, OnScrubPositionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		PreviewData = InArgs._PreviewData;
		OnContextMenuOpening = InArgs._OnContextMenuOpening;
		OnScrubPositionChanged = InArgs._OnScrubPositionChanged;
		
		ChildSlot
		[
			SNew(SOverlay)
			.Visibility(EVisibility::SelfHitTestInvisible)
			
			+ SOverlay::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(EVisibility::HitTestInvisible)
				.Text(LOCTEXT("NoTrajectoryInTimelineLabel", "No trajectory selected or available"))
				.Justification(ETextJustify::Center)
				.Visibility_Lambda([this](){ return PreviewData->OutputTrajectory.Samples.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
			
			+ SOverlay::Slot()
			[
				SAssignNew(Timeline, SSimpleTimeSlider)
				.IsEnabled_Lambda([this](){ return !PreviewData->OutputTrajectory.Samples.IsEmpty(); })
				.DesiredSize({100,24})
				.ClampRangeHighlightSize(0.15f)
				.ClampRangeHighlightColor(FLinearColor::Red.CopyWithNewOpacity(0.5f))
				.ScrubPosition_Lambda([this](){ return PreviewData->Timeline.ScrubTime; })
				.ViewRange_Lambda([this]()
				{
					TRange<double> OutputRange(0, 0);

					if (!PreviewData->OutputTrajectory.Samples.IsEmpty())
					{
						OutputRange.SetLowerBoundValue(PreviewData->OutputTrajectory.Samples[0].Time);
						OutputRange.SetUpperBoundValue(PreviewData->OutputTrajectory.Samples.Last().Time);
					}
					
					return OutputRange;
				})
				.ClampRange_Lambda([this]()
				{
					TRange<double> OutputRange(0, 0);

					if (!PreviewData->OutputTrajectory.Samples.IsEmpty())
					{
						OutputRange.SetLowerBoundValue(PreviewData->OutputTrajectory.Samples[0].Time);
						OutputRange.SetUpperBoundValue(PreviewData->OutputTrajectory.Samples.Last().Time);
					}
					
					return OutputRange;
				})
				.OnEndScrubberMovement_Lambda([this]()
				{
					PreviewData->Timeline.bShouldDrawScrubber = Timeline->IsHovered();
				})
				.OnScrubPositionChanged_Lambda([this](double NewScrubTime, bool bIsScrubbing)
				{
					if (bIsScrubbing)
					{
						PreviewData->Timeline.ScrubTime = NewScrubTime;
					}

					PreviewData->Timeline.bShouldDrawScrubber = true;
					
					OnScrubPositionChanged.ExecuteIfBound(NewScrubTime, bIsScrubbing);
				})
			]
		];
	}

	/** Begin SWidget */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		PreviewData->Timeline.bShouldDrawScrubber = true;
	}
		
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		PreviewData->Timeline.bShouldDrawScrubber = false;
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		//@todo: Look into why this doesnt work?
		const bool bIsRightKey = InKeyEvent.GetKey() == EKeys::Right;
		const bool bIsLeftKey = InKeyEvent.GetKey() == EKeys::Left;
		
		if ((bIsRightKey || bIsLeftKey) && IsValid(PreviewData->Details))
		{
			const float Multiplier = static_cast<float>(bIsRightKey) - static_cast<float>(bIsLeftKey);
			PreviewData->Timeline.ScrubTime += PreviewData->Details->ExportSettings.FrameRate.AsDecimal() * Multiplier;
			
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent) override
	{
		const FVector2f SummonLocation = InMouseEvent.GetScreenSpacePosition();
		const bool bIsRightMouseButtonDown = InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
		const bool bContextMenuOpeningBound = OnContextMenuOpening.IsBound();
		
		if (bIsRightMouseButtonDown)
		{
			if (bContextMenuOpeningBound)
			{
				// Get the context menu content. If NULL, don't open a menu.
				const TSharedPtr<SWidget> MenuContent = OnContextMenuOpening.Execute();

				if (MenuContent.IsValid())
				{
					const FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
					FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
				}
			}

			// Note that we intentionally not handle the event so that the SSimpleTimeSlider is still able to handle its OnPreviewMouseButtonDown().
		}

		return FReply::Unhandled();
	}

	static void PaintBoneSection(const FGeometry& AllottedGeometry, const FSlateFontInfo& InFont, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const SSimpleTimeSlider::FScrubRangeToScreen& InRangeToScreen, uint32 InBoneCount, float InStartTime, float InEndTime, int InSectionId)
	{
		float LeftSection = InRangeToScreen.InputToLocalX(InStartTime);
		float RightSection = InRangeToScreen.InputToLocalX(InEndTime);
		
		float SectionSize = RightSection - LeftSection;
		float SectionHorizontalPosition = LeftSection;

		FString SectionString = FString::Format(TEXT("Bone Count: {0}"), { InBoneCount });
		TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FVector2D TextSize = FontMeasureService->Measure(SectionString, InFont);
		FVector2f TextOffset = FVector2f(SectionSize * 0.5f - TextSize.X * 0.5f);
		FLinearColor BackgroundColor = (FLinearColor::MakeRandomSeededColor(1999 + InSectionId) * 0.3f).ToFColorSRGB().WithAlpha(255);
		float Height = AllottedGeometry.GetLocalSize().Y * 0.5f;
		
		FPaintGeometry RangeGeometry;
		RangeGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(SectionSize, Height), FSlateLayoutTransform(FVector2f(SectionHorizontalPosition, -AllottedGeometry.GetLocalSize().Y + Height)));
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			InLayerId + 1,
			RangeGeometry,
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			BackgroundColor
			);

		RangeGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(SectionSize, Height), FSlateLayoutTransform(FVector2f(SectionHorizontalPosition + TextOffset.X, -AllottedGeometry.GetLocalSize().Y + Height)));
		FSlateDrawElement::MakeText(
			OutDrawElements,
			InLayerId + 2,
			RangeGeometry,
			SectionString,
			InFont,
			ESlateDrawEffect::None,
			FLinearColor::White
		);
	}
	
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		// Draw normal widget.
		LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		// Draw bone count splits.
		if (PreviewData->SourceIndex != INDEX_NONE)
		{
			const FGameplayTrajectory SourceTrajectory = PreviewData->SourceTrajectories[PreviewData->SourceIndex];

			if (!SourceTrajectory.TraceInfo.IsEmpty())
			{
				const bool bEnabled = bParentEnabled;
				const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

				TRange<double> LocalViewRange = Timeline->GetTimeRange();
				const float LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
				const float LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
				const float LocalSequenceLength = LocalViewRangeMax-LocalViewRangeMin;
				FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);
				
				FVector2D Scale = FVector2D(1.0f,1.0f);
				if ( LocalSequenceLength > 0)
				{
					SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.GetLocalSize() );
			
					// Draw section
					// TRange<double> ClampRange = Timeline->GetClampRange();

					for (int i = 0; i < SourceTrajectory.TraceInfo.Num(); ++i)
					{
						int CurrentSplitIndex = SourceTrajectory.TraceInfo.Ranges[i].GetLowerBoundValue();
						int NextSplitIndex = SourceTrajectory.TraceInfo.Ranges[i].GetUpperBoundValue();

						const TArray<FTransform>& CurrentPoseRef = SourceTrajectory.Poses[CurrentSplitIndex];
						const TArray<FTransform>& NextPoseRef = SourceTrajectory.Poses[NextSplitIndex];

						const FGameplayTrajectory::FSample& CurrentSampleRef = SourceTrajectory.Samples[CurrentSplitIndex];
						const FGameplayTrajectory::FSample& NextSampleRef = SourceTrajectory.Samples[NextSplitIndex];
						
						PaintBoneSection(AllottedGeometry, SmallLayoutFont, OutDrawElements, LayerId, RangeToScreen, CurrentPoseRef.Num(), CurrentSampleRef.Time, NextSampleRef.Time, i);
					}
				}	
			}
		
		}
		
		return LayerId;
	}
	/** End SWidget */

private:

	/** Used to draw the trajectory */
	TSharedPtr<FPreviewData> PreviewData;

	/** Trajectory timeline widget */
	TSharedPtr<SSimpleTimeSlider> Timeline;
	
	/** Delegate to invoke when the context menu should be opening. If it is nullptr, a context menu will not be summoned. */
	FOnContextMenuOpening OnContextMenuOpening;

	/** Delegate to invoke when the scrub position was changed. */
	SSimpleTimeSlider::FOnScrubPositionChanged OnScrubPositionChanged;
};

void SExportTrajectoriesWindow::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;
	Trajectories.Append(InArgs._Trajectories);
	TrajectoryOwnerNames.Append(InArgs._OwnerNames);
	PreviewData = MakeShared<FPreviewData>();

	check(Trajectories.Num() == TrajectoryOwnerNames.Num());
	
	ImmutableState.SkeletalMeshInfos = InArgs._SkelMeshInfos;
	ImmutableState.DebugInfos = InArgs._DebugInfos;
	
	check(Trajectories.Num() == ImmutableState.DebugInfos.Num())
	
	// Export configuration settings.
	FName Name = FName("TrajectoryExportDetails");
	Name = MakeUniqueObjectName(GetTransientPackage(), UTrajectoryExportDetails::StaticClass(), Name, EUniqueObjectNameOptions::GloballyUnique);
	ExportDetails = NewObject<UTrajectoryExportDetails>( GetTransientPackage(), Name, RF_Public | RF_Standalone);

	// Allow user to view and edit the configuration settings.
	{
		FDetailsViewArgs GridDetailsViewArgs;
		GridDetailsViewArgs.bAllowSearch = false;
		GridDetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		GridDetailsViewArgs.bHideSelectionTip = true;
		GridDetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		GridDetailsViewArgs.bShowOptions = false;
		GridDetailsViewArgs.bAllowMultipleTopLevelObjects = false;

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		ExportDetailsView = PropertyEditorModule.CreateDetailView(GridDetailsViewArgs);
		ExportDetailsView->SetObject(ExportDetails.Get());
		ExportDetailsView->OnFinishedChangingProperties().AddSP(this, &SExportTrajectoriesWindow::OnFinishedChangingExportSettingsSelectionProperties);	
	}
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			// Settings / Export
			
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			/*.ResizeMode(ESplitterResizeMode::Fill)*/

			// Viewport / Slider
			
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.6)
				[
					SAssignNew(Viewport, SPreviewTrajectoryViewport)
					.PreviewData(PreviewData)
				]
				
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(ViewportTimeline, STrajectoryTimeline)
					.PreviewData(PreviewData)
					.OnScrubPositionChanged_Lambda([this](double NewSliderTimer, bool bIsScrubbing)
					{
						if (bIsScrubbing)
						{
							Viewport->ResetPreviewSkeletalMesh();
						}
					})
					.OnContextMenuOpening_Lambda([InPreviewData = PreviewData, InExportWindow = this]()
					{
						// Early out
						if (!InPreviewData.IsValid() || !IsValid(InPreviewData->Details))
						{
							return TSharedPtr<SWidget>();
						}
						
						const int32 ScrubSampleIndex = GetSampleIndexFromTime(InPreviewData->Details->ExportSettings.FrameRate, InPreviewData->Timeline.ScrubTime, InPreviewData->OutputTrajectory.Samples.Num());
						const double ScrubSampleTime = InPreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Time;
						
						FMenuBuilder MenuBuilder(true, nullptr);

						// Set min range via timeline
						{
							const FText Label = FText::Format(LOCTEXT("SetExportRangeMinLabel", "Set export range min at frame {0}* ({1}s)"), ScrubSampleIndex, ScrubSampleTime); 
							const FText ToolTip = LOCTEXT("SetExportRangeMinToolTip", "Sets the ExportRange's Min value using the current scrub frame's time.");
							FUIAction Action = FExecuteAction::CreateLambda([InPreviewData, InExportWindow]()
							{
								if (IsValid(InPreviewData->Details))
								{
									const int32 ScrubSampleIndex = InPreviewData->Details->ExportSettings.FrameRate.AsFrameTime(InPreviewData->Timeline.ScrubTime).FloorToFrame().Value;
									InPreviewData->Details->ExportSettings.Range.Min = InPreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Time;
									
									// @todo: Should this just be done in PostEditChange()?
									InPreviewData->Details->PlayLength = InPreviewData->Details->ExportSettings.Range.Size();
									InPreviewData->Details->NumberOfKeyFrames = InPreviewData->Details->ExportSettings.FrameRate.AsFrameTime(InPreviewData->Details->PlayLength).CeilToFrame().Value;
									InPreviewData->Details->ExportSettings.OriginTime = FMath::Clamp(InPreviewData->Details->ExportSettings.OriginTime, InPreviewData->Details->ExportSettings.Range.Min, InPreviewData->Details->ExportSettings.Range.Max);

									// Update asset info (since we care about which trace range we use)
									InExportWindow->UpdateAssetInfo(InPreviewData->Details->ExportAssetInfo.Skeleton, InPreviewData->Details->ExportAssetInfo.SkeletalMesh);
								}
							});
							
							MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), Action);
						}

						// Set max range via timeline
						{
							const FText Label = FText::Format(LOCTEXT("SetExportRangeMaxLabel", "Set export range max at frame {0}* ({1}s)"), ScrubSampleIndex, ScrubSampleTime); 
							const FText ToolTip = LOCTEXT("SetExportRangeMaxToolTip", "Sets the ExportRange's Max value using the current scrub frame's time.");
							FUIAction Action = FExecuteAction::CreateLambda([InPreviewData, InExportWindow]()
							{
								if (IsValid(InPreviewData->Details))
								{
									const int32 ScrubSampleIndex = InPreviewData->Details->ExportSettings.FrameRate.AsFrameTime(InPreviewData->Timeline.ScrubTime).FloorToFrame().Value;
									InPreviewData->Details->ExportSettings.Range.Max = InPreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Time;

									// @todo: Should this just be done in PostEditChange()?
									InPreviewData->Details->PlayLength = InPreviewData->Details->ExportSettings.Range.Size();
									InPreviewData->Details->NumberOfKeyFrames = InPreviewData->Details->ExportSettings.FrameRate.AsFrameTime(InPreviewData->Details->PlayLength).CeilToFrame().Value;
									InPreviewData->Details->ExportSettings.OriginTime = FMath::Clamp(InPreviewData->Details->ExportSettings.OriginTime, InPreviewData->Details->ExportSettings.Range.Min, InPreviewData->Details->ExportSettings.Range.Max);

									// Update asset info (since we care about which trace range we use)
									InExportWindow->UpdateAssetInfo(InPreviewData->Details->ExportAssetInfo.Skeleton, InPreviewData->Details->ExportAssetInfo.SkeletalMesh);
								}
							});
							
							MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), Action);
						}

						// Set origin via timeline
						{
							const FText Label = FText::Format(LOCTEXT("SetThisFrameAsTheOriginLabel", "Set origin at frame {0}* ({1}s)"), ScrubSampleIndex, ScrubSampleTime); 
							const FText ToolTip = LOCTEXT("SetExportOriginToolTip", "Sets the current scrub frame at the origin in baked out asset.");
							FUIAction Action = FExecuteAction::CreateLambda([InPreviewData]()
							{
								if (IsValid(InPreviewData->Details))
								{
									const int32 ScrubSampleIndex = InPreviewData->Details->ExportSettings.FrameRate.AsFrameTime(InPreviewData->Timeline.ScrubTime).FloorToFrame().Value;
									const double ScrubSampleTime = InPreviewData->OutputTrajectory.Samples[ScrubSampleIndex].Time;

									InPreviewData->Details->ExportSettings.bShouldForceOrigin = true;
									InPreviewData->Details->ExportSettings.OriginTime = FMath::Clamp(ScrubSampleTime, InPreviewData->Details->ExportSettings.Range.Min, InPreviewData->Details->ExportSettings.Range.Max);
								}
							});
							
							MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), Action);
						}
						
						return MenuBuilder.MakeWidget().ToSharedPtr();
					})
				]	
			]
			
			+ SSplitter::Slot()
			.SizeRule(SSplitter::SizeToContent)
			.Value(0.4f)
			[
				SNew(SVerticalBox)

				// Trajectory Picker Row
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.AutoHeight()
				[
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SBorder)
						.Padding(2.0f)
						.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
						[
							SNew(SHorizontalBox)

							// Trajectory to export label
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(6.0f, 3.0f)
							.AutoWidth()
							[
								SNew(SRichTextBlock)
								.Text(LOCTEXT("TrajectoryPickerLabel", "Source"))
								.TransformPolicy(ETextTransformPolicy::ToUpper)
								.DecoratorStyleSet(&FAppStyle::Get())
								.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
								.ToolTipText(LOCTEXT("TrajectoryPickerTooltip", "Traced Skeletal Mesh Component to use as the export source for a sequence or trajectory."))
							]

							// Trajectory picker widget
							+ SHorizontalBox::Slot()
							.Padding(3.0f, 3.0f)
							.AutoWidth()
							[
								SAssignNew(TrajectoryPickerComboButton, SComboButton)
								.VAlign(VAlign_Center)
								.OnGetMenuContent(this, &SExportTrajectoriesWindow::OnGetTrajectoryPickerMenuContent)
								.OnMenuOpenChanged(this, &SExportTrajectoriesWindow::OnTrajectoryPickerMenuOpened)
								.ButtonContent()
								[
									SNew(STextBlock)
									.Justification(ETextJustify::Center)
									.Text_Lambda([this](){ return SelectedTrajectoryIndex == INDEX_NONE ? FText::FromName(NAME_None): FText::FromName(TrajectoryOwnerNames[SelectedTrajectoryIndex]); })
								]
							]
						]
					]
				]

				// Settings
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillHeight(0.6f)
				[
					SNew(SBorder)
					.Padding(0)
					[
						ExportDetailsView.ToSharedRef()
					]
				]

				// Warning
				+SVerticalBox::Slot()
				.Padding(2)
				.AutoHeight()
				[
					SNew(SWarningOrErrorBox)
					.Visibility_Lambda([this]{ return GetExportValidation() != EExportValidation::Valid ? EVisibility::Visible : EVisibility::Collapsed; })
					.MessageStyle(EMessageStyle::Warning)
					.Message_Lambda([this] { return GetWarningMessage(); })
				]
				
				// Export
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.FillHeight(0.2f)
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(6.0f)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SAssignNew(ExportButton, SButton)
						.ButtonStyle(FAppStyle::Get(), "Button")
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.ToolTipText(LOCTEXT("BakeTrajectoryButtonToolTip", "Bake out trajectory to the target asset"))
						.HAlign(HAlign_Center)
						.IsEnabled_Lambda([this]()
						{
							return GetExportValidation() == EExportValidation::Valid;
						})
						.OnClicked_Lambda([this]()
						{
							// Export trajectories to assets.
							UTrajectoryExportOperation::ExportTrajectory(&Trajectories[SelectedTrajectoryIndex], ExportDetails->ExportSettings, ExportDetails->ExportAssetInfo, TrajectoryOwnerNames[SelectedTrajectoryIndex].ToString());
							
							return FReply::Handled();
						})
						.Text(LOCTEXT("ExportLabel", "Export"))
					]
				]
			]
			
		]
	];
	
	// Now that everything is constructed properly, default to first trajectory as selected, if we have any.
	if (!Trajectories.IsEmpty())
	{
		// Select first valid trajectory
		for (int i = 0; i < Trajectories.Num(); ++i)
		{
			const bool bHasAnyData = !Trajectories[i].Samples.IsEmpty() && !Trajectories[i].Poses.IsEmpty();
			
			if (bHasAnyData)
			{
				SelectedTrajectoryIndex = i;
				break;
			}
		}
		
		if (SelectedTrajectoryIndex != INDEX_NONE)
		{
			UpdateViewportAfterTrajectorySelection();
		}
	}
}

void SExportTrajectoriesWindow::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ExportDetails);
}

void SExportTrajectoriesWindow::UpdateViewportAfterTrajectorySelection()
{
	// Reset export settings
	ExportDetails->Reset();
	PreviewData->Details = ExportDetails;

	// Update which trajectory we are selecting based on the index
	PreviewData->SourceIndex = SelectedTrajectoryIndex;
	PreviewData->Viewport.SourceSkeletalMeshInfo = ImmutableState.SkeletalMeshInfos[SelectedTrajectoryIndex];
	PreviewData->Viewport.SourceSkeletalMeshComponentId = ImmutableState.DebugInfos[SelectedTrajectoryIndex].OwnerId;

	// Allow to view raw data.
	PreviewData->SourceTrajectories = MakeConstArrayView(Trajectories);
				
	// Match export frame rate
	PreviewData->OutputTrajectory.Samples.Reset();
	PreviewData->OutputTrajectory.Poses.Reset();
	PreviewData->OutputTrajectory.TraceInfo.Reset();
	FTrajectoryToolsLibrary::TransformTrajectoryToMatchFrameRate(Trajectories[SelectedTrajectoryIndex], ExportDetails->ExportSettings.FrameRate, PreviewData->OutputTrajectory);

	// Always start with entire trajectory range marked for available for export
	if (!PreviewData->OutputTrajectory.Samples.IsEmpty())
	{
		ExportDetails->ExportSettings.Range.Min = PreviewData->OutputTrajectory.Samples[0].Time;
		ExportDetails->ExportSettings.Range.Max = PreviewData->OutputTrajectory.Samples.Last().Time;

		ExportDetails->NumberOfKeyFrames = PreviewData->OutputTrajectory.Samples.Num();
		ExportDetails->PlayLength = ExportDetails->ExportSettings.Range.Size();
	}

	// Update asset info
	UpdateAssetInfo(ExportDetails->ExportAssetInfo.Skeleton, ExportDetails->ExportAssetInfo.SkeletalMesh);
				
	// Focus on preview trajectory
	PreviewViewportFocusOnTrajectory(PreviewData, Viewport->GetViewportClient());

	// Reset viewport.
	Viewport->ResetPreviewSkeletalMesh();
}

TSharedRef<SWidget> SExportTrajectoriesWindow::OnGetTrajectoryPickerMenuContent()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection(NAME_None, FText::GetEmpty());
	
	for (int i = 0; i < TrajectoryOwnerNames.Num(); ++i)
	{
		const bool bIsValidTrajectory = Trajectories[i].IsValid();
		const FText ToolTip = bIsValidTrajectory ? FText::FromName(TrajectoryOwnerNames[i]) : LOCTEXT("InvalidTrajectoryTooltip", "Invalid. Trajectory data is empty or there is a mismatch between poses and samples.");

		FUIAction UIAction;

		UIAction.CanExecuteAction = FCanExecuteAction::CreateLambda([i, this]()
		{
			const bool bHasAnyData = !Trajectories[i].Samples.IsEmpty() && !Trajectories[i].Poses.IsEmpty();
			return Trajectories[i].IsValid() && bHasAnyData;
		});
		
		UIAction.ExecuteAction = FExecuteAction::CreateLambda([i, this]()
		{
			const int PrevSelectedTrajectoryIndex = SelectedTrajectoryIndex;
			SelectedTrajectoryIndex = i;

			if (PrevSelectedTrajectoryIndex != SelectedTrajectoryIndex && SelectedTrajectoryIndex != INDEX_NONE)
			{
				UpdateViewportAfterTrajectorySelection();
			}
		});
		
		MenuBuilder.AddMenuEntry(FText::FromName(TrajectoryOwnerNames[i]), ToolTip, FSlateIcon(), UIAction);
	}
	
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SExportTrajectoriesWindow::OnTrajectoryPickerMenuOpened(bool bIsOpen)
{
	if (!bIsOpen)
	{
		TrajectoryPickerComboButton->SetMenuContent(SNullWidget::NullWidget);
	}
}

void SExportTrajectoriesWindow::OnFinishedChangingExportSettingsSelectionProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!PropertyChangedEvent.Property ||  PropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet)
	{
		return;
	}
	
	const bool ChangedTargetFrameRate = PropertyChangedEvent.Property->GetName() == "FrameRate";
    if (ChangedTargetFrameRate)
    {
    	// Clear preview / output trajectory.
    	PreviewData->OutputTrajectory.Samples.Reset();
    	PreviewData->OutputTrajectory.Poses.Reset();
    	
    	// Trigger gameplay trajectory recompute.
    	FTrajectoryToolsLibrary::TransformTrajectoryToMatchFrameRate(Trajectories[SelectedTrajectoryIndex], ExportDetails->ExportSettings.FrameRate, PreviewData->OutputTrajectory);
    }

    const bool ChangedExportRange = PropertyChangedEvent.Property->GetName() == "Min" || PropertyChangedEvent.Property->GetName() == "Max";
    if (ChangedExportRange && IsValid(PreviewData->Details))
    {
    	// Range can't be bigger than play length
    	PreviewData->Details->ExportSettings.Range.Min = FMath::Clamp(PreviewData->Details->ExportSettings.Range.Min, 0, PreviewData->OutputTrajectory.Samples.Last().Time);
    	PreviewData->Details->ExportSettings.Range.Max = FMath::Clamp(PreviewData->Details->ExportSettings.Range.Max, 0, PreviewData->OutputTrajectory.Samples.Last().Time);

    	// Origin can't be outside of range
    	PreviewData->Details->ExportSettings.OriginTime = FMath::Clamp(PreviewData->Details->ExportSettings.OriginTime, PreviewData->Details->ExportSettings.Range.Min, PreviewData->Details->ExportSettings.Range.Max);

    	// Update asset info (since we care about which trace range we use)
    	UpdateAssetInfo(ExportDetails->ExportAssetInfo.Skeleton, ExportDetails->ExportAssetInfo.SkeletalMesh);
    }
}

FText SExportTrajectoriesWindow::GetWarningMessage() const
{
	const EExportValidation ExportValidationState = GetExportValidation();
	
	switch (ExportValidationState)
	{
	case EExportValidation::MultipleTraceRanges: return LOCTEXT("MultipleTraceRangeLabel", "Multiple trace ranges");
	case EExportValidation::NoTrajectorySelected: return  LOCTEXT("NoTrajectorySelectedLabel", "No trajectory selected");
	case EExportValidation::MissingAssetInfo: return LOCTEXT("MissingAssetInfoLabel", "Missing asset information");
	case EExportValidation::ExportPathIsInvalid: return LOCTEXT("ExportPathIsInvalidLabel", "Invalid directory");
	case EExportValidation::AssetAlreadyExists: return LOCTEXT("AssetAlreadyExistsLabel", "Asset already exists");
	case EExportValidation::Valid:
	default:
		return FText::GetEmpty();
	}
}

SExportTrajectoriesWindow::EExportValidation SExportTrajectoriesWindow::GetExportValidation() const
{
	// Early out if export range contains multiple trace ranges.
	if (PreviewData->OutputTrajectory.TraceInfo.Num() > 1)
	{
		int MinIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.Range.Min, PreviewData->OutputTrajectory.Samples.Num());
		int MaxIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.Range.Max, PreviewData->OutputTrajectory.Samples.Num());

		FTrajectoryToolsLibrary::FRangeOverlapTestResult TestResult{};
		FTrajectoryToolsLibrary::GetRangeOverlaps(PreviewData->OutputTrajectory, TRange<int32>{MinIndex, MaxIndex}, TestResult);

		if (TestResult.bOverlaps && TestResult.Ranges.Num() > 1)
		{
			return EExportValidation::MultipleTraceRanges;
		}
	}

	if (SelectedTrajectoryIndex == INDEX_NONE)
	{
		return EExportValidation::NoTrajectorySelected;
	}

	if (ExportDetails == nullptr || !ExportDetails->ExportAssetInfo.IsValid())
	{
		return EExportValidation::MissingAssetInfo;	
	}

	if (!ExportDetails->ExportAssetInfo.IsExportPathValid())
	{
		return EExportValidation::ExportPathIsInvalid;
	}

	if (!ExportDetails->ExportSettings.bShouldOverwriteExistingFiles && ExportDetails->ExportAssetInfo.DoesAssetExistAtPath())
	{
		return EExportValidation::AssetAlreadyExists;
	}
	
	return EExportValidation::Valid;
}

void SExportTrajectoriesWindow::UpdateAssetInfo(FSoftObjectPath&  OutSkeletonPath, FSoftObjectPath & OutSkeletalMeshPath) const
{
	if (!ImmutableState.SkeletalMeshInfos.IsValidIndex(SelectedTrajectoryIndex) || !IRewindDebugger::Instance() || !IRewindDebugger::Instance()->GetAnalysisSession())
	{
		OutSkeletonPath.Reset();
		OutSkeletalMeshPath.Reset();
		return;
	}
	
	// Get data providers for traced objets.
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();
	const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
	const IAnimationProvider* AnimationProvider = Session->ReadProvider<IAnimationProvider>("AnimationProvider");
	
	if (GameplayProvider && AnimationProvider)
	{
		// Preview using skeletal mesh from current trace range.
		if (PreviewData->OutputTrajectory.TraceInfo.Num() > 1)
		{
			int MinIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.Range.Min, PreviewData->OutputTrajectory.Samples.Num());
			int MaxIndex = GetSampleIndexFromTime(PreviewData->Details->ExportSettings.FrameRate, PreviewData->Details->ExportSettings.Range.Max, PreviewData->OutputTrajectory.Samples.Num());

			FTrajectoryToolsLibrary::FRangeOverlapTestResult TestResult{};
			FTrajectoryToolsLibrary::GetRangeOverlaps(PreviewData->OutputTrajectory, TRange<int32>{MinIndex, MaxIndex}, TestResult);

			if (TestResult.bOverlaps && TestResult.Ranges.Num() == 1)
			{
				FSkeletalMeshInfo SkeletalMeshInfo = PreviewData->OutputTrajectory.TraceInfo.SkeletalMeshInfos[TestResult.Ranges.Last()];
			
				// Ensure our skeleton is valid.
				if (SkeletalMeshInfo.SkeletonId != 0)
				{
					TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
					const FObjectInfo& SkeletonObjectInfo = GameplayProvider->GetObjectInfo(SkeletalMeshInfo.SkeletonId);
					OutSkeletonPath = FSoftObjectPath(SkeletonObjectInfo.PathName);
					
					return;
				}
			}
		}

		// Preview using initial traced skeletal mesh.
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		if (const FSkeletalMeshInfo* SkeletalMeshInfo = AnimationProvider->FindSkeletalMeshInfo(ImmutableState.SkeletalMeshInfos[SelectedTrajectoryIndex].Id))
		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			if (const FObjectInfo* SkeletalMeshObjectInfo = GameplayProvider->FindObjectInfo(ImmutableState.SkeletalMeshInfos[SelectedTrajectoryIndex].Id))
			{
				OutSkeletalMeshPath = FSoftObjectPath(SkeletalMeshObjectInfo->PathName);
				
				// Ensure our skeleton is valid.
				if (SkeletalMeshInfo->SkeletonId != 0)
				{
					const FObjectInfo& SkeletonObjectInfo = GameplayProvider->GetObjectInfo(SkeletalMeshInfo->SkeletonId);
					OutSkeletonPath = FSoftObjectPath(SkeletonObjectInfo.PathName);
					
					return;
				}
			}
		}
	}

	OutSkeletonPath.Reset();
	OutSkeletalMeshPath.Reset();
}

#undef LOCTEXT_NAMESPACE
