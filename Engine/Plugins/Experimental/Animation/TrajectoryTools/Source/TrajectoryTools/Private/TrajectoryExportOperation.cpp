// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrajectoryExportOperation.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Misc/ITransaction.h"
#include "ObjectTools.h"
#include "TrajectoryLibrary.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Factories/AnimSequenceFactory.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TrajectoryExportOperation)

#define LOCTEXT_NAMESPACE "TrajectoryLibrary"

// @todo: Need to use the #if EDITOR around code that needs the editor (transactions).

FTrajectoryExportSettings::FTrajectoryExportSettings()
{
	Reset();
}

bool FTrajectoryExportSettings::IsValid() const
{
	const bool bValidFrameRate = FrameRate.IsValid();
	const bool bValidRange = Range.IsValid() && Range.Size() != 0;
	const bool bValidOriginTime = bShouldForceOrigin ? Range.Contains(OriginTime) : true;
	
	return bValidFrameRate && bValidRange && bValidOriginTime;
}

void FTrajectoryExportSettings::Reset()
{
	FrameRate = FFrameRate(30, 1);
	Range = {0, 0};
	bShouldForceOrigin = false;
	OriginTime = 0;
	bShouldOverwriteExistingFiles = false;
	bShouldExportOnlyAnimatedBones = true;
}

bool FTrajectoryExportAssetInfo::IsExportPathValid() const
{
	FString ObjectPath = FolderPath.Path / AssetName;
	
	static const FString GameDir = TEXT("/Game/");
	static const FString EngineDir = TEXT("/Engine/");
	static const FString PluginsDir = TEXT("/Plugins/");

	bool bDirectoryExists = true;
	
	if (ObjectPath.StartsWith(GameDir))
	{
		FString ProjectDir = FPaths::ProjectDir();
		const FString RelativeAssetFilename = FPaths::Combine(FPaths::ProjectContentDir(), ObjectPath.Replace(*GameDir, TEXT("")));
		const FString RelativeAssetPath = FPaths::GetPath(RelativeAssetFilename);
		
		bDirectoryExists = FPaths::DirectoryExists(FPaths::ConvertRelativePathToFull(RelativeAssetPath));
	}
	else if (ObjectPath.StartsWith(EngineDir))
	{
		FString ProjectDir = FPaths::ProjectDir();
		const FString RelativeAssetFilename = FPaths::Combine(FPaths::EngineContentDir(), ObjectPath.Replace(*EngineDir, TEXT("")));
		const FString RelativeAssetPath = FPaths::GetPath(RelativeAssetFilename);
		
		bDirectoryExists = FPaths::DirectoryExists(FPaths::ConvertRelativePathToFull(RelativeAssetPath));
	}
	
	return bDirectoryExists;
}

bool FTrajectoryExportAssetInfo::DoesAssetExistAtPath() const
{
	FString ObjectPath = FolderPath.Path / AssetName;
	
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
	const bool bAssetExists = AssetData.IsValid();
	
	return bAssetExists;
}

bool FTrajectoryExportAssetInfo::CanCreateAsset() const
{
	return IsExportPathValid() && !DoesAssetExistAtPath();
}

void FTrajectoryExportAssetInfo::Reset()
{
	AssetName = "";
	FolderPath.Path = TEXT("");
	Skeleton.Reset();
	SkeletalMesh.Reset();
}

bool FTrajectoryExportAssetInfo::IsValid() const
{
	return Skeleton.IsAsset() && SkeletalMesh.IsAsset() && !AssetName.IsEmpty() && !FolderPath.Path.IsEmpty();
}

void FTrajectoryExportContext::Reset()
{
	Settings.Reset();
	AssetInfo.Reset();
	SourceObjectName.Reset();
	Data = nullptr;
}

bool FTrajectoryExportContext::IsValid() const
{
	return Settings.IsValid() && AssetInfo.IsValid() && Data != nullptr;
}

void UTrajectoryExportOperation::ExportTrajectory(FGameplayTrajectory* InTrajectory, const FTrajectoryExportSettings& InSettings, const FTrajectoryExportAssetInfo& InAssetInfo, const FString& InSourceObjectName)
{
	FTrajectoryExportContext Context;
	Context.Data = InTrajectory;
	Context.Settings = InSettings;
	Context.AssetInfo = InAssetInfo;
	Context.SourceObjectName = InSourceObjectName;
	
	// Actually run the batch operation
	const TStrongObjectPtr BatchOperation(NewObject<UTrajectoryExportOperation>());
	BatchOperation->Run(Context);
}

void UTrajectoryExportOperation::Run(const FTrajectoryExportContext& Context)
{
	Reset();

	// @todo: Validate context and log errors.
	
	// Keep track of progress.
	constexpr int32 NumProgressSteps = 6; // Gen Assets: 1, Export To Assets: 4, Notify User: 1.
	FScopedSlowTask Progress(NumProgressSteps, LOCTEXT("ExportingOperationProgress", "Exporting assets..."));

	// Show progress dialog.
	constexpr bool bShowCancelButton = true;
	Progress.MakeDialog(bShowCancelButton);

	// Wrap all changes into a single commit while running the export operation.
	ActiveTransaction = MakeUnique<FScopedTransaction>(TEXT("TrajectoryExportOperation"), LOCTEXT("TrajectoryExportOperationTransaction", "Exporting trajectories"), nullptr);

	// Start exporting
	GenerateAssets(Context, Progress);
	ExportDataToAssets(Context, Progress);
	NotifyUserOfResults(Context, Progress);
	CleanupIfCancelled(Progress);

	// Reset our open transaction to commit it now that the operation is completed.
	ActiveTransaction.Reset();
}

void UTrajectoryExportOperation::Reset()
{
	AssetToProcess = nullptr;
	GeneratedAsset = nullptr;
	ActiveTransaction.Reset();
}

void UTrajectoryExportOperation::GenerateAssets(const FTrajectoryExportContext& Context, FScopedSlowTask& Progress)
{
	Progress.EnterProgressFrame(1.0f, LOCTEXT("ExportOperationProgress_GenerateAssets", "Generating asset(s)..."));
	
	FString FinalPackageName;
	FString FinalAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	
	// Get unique names before creating the respective package and asset(s).
	{
		FString PackagePath = Context.AssetInfo.FolderPath.Path.IsEmpty() ? TEXT("/Game") : Context.AssetInfo.FolderPath.Path;
		FString BasePackagePath = Context.AssetInfo.FolderPath.Path + "/"; //Context.AssetInfo.AssetName;
		
		AssetToolsModule.Get().CreateUniqueAssetName(BasePackagePath, Context.AssetInfo.AssetName, FinalPackageName,FinalAssetName);		
	}

	// Attempt to overwrite asset(s) if possible.
	if (Context.Settings.bShouldOverwriteExistingFiles)
	{
		UPackage* ExistingPackage = FindPackage(nullptr, *FinalPackageName);
		UAnimSequence* ExistingObject = Cast<UAnimSequence>(StaticFindObject(UAnimSequence::StaticClass(), ExistingPackage, *FinalAssetName));
		
		if (ExistingObject)
		{
			// store in batch operation variable
			AssetToProcess = ExistingObject;
			return;
		}
	}

	if (FinalPackageName.EndsWith(FinalAssetName))
	{
		FinalPackageName = FinalPackageName.LeftChop(FinalAssetName.Len() + 1);
	}

	// Load traced skeleton data.
	USkeleton* Skeleton = TSoftObjectPtr<USkeleton>(Context.AssetInfo.Skeleton).LoadSynchronous();
	USkeletalMesh* SkeletalMesh =  TSoftObjectPtr<USkeletalMesh>(Context.AssetInfo.SkeletalMesh).LoadSynchronous();

	const bool bIsSkeletalMeshValid = IsValid(SkeletalMesh);
	const bool bIsSkeletonValid = IsValid(SkeletalMesh);
	
	if (!bIsSkeletonValid && bIsSkeletalMeshValid)
	{
		Skeleton = SkeletalMesh->GetSkeleton();
	}
	else if (!bIsSkeletalMeshValid && bIsSkeletonValid)
	{
		SkeletalMesh = Skeleton->GetPreviewMesh();
	}
	else if (!bIsSkeletalMeshValid && !bIsSkeletonValid)
	{
		checkNoEntry();
	}
	
	// Create new asset for storing trajectory data.
	UAnimSequenceFactory* Factory = NewObject<UAnimSequenceFactory>();
	Factory->TargetSkeleton = Skeleton;
	Factory->PreviewSkeletalMesh = SkeletalMesh;
	UAnimSequence* NewAsset = Cast<UAnimSequence>(AssetToolsModule.Get().CreateAsset(*FinalAssetName, *FinalPackageName/* FPackageName::GetLongPackagePath(FinalPackageName)*/, UAnimSequence::StaticClass(), Factory));

	check(IsValid(NewAsset))
	
	GeneratedAsset = NewAsset;
	AssetToProcess = GeneratedAsset;

	// Inform asset registry of our new asset.
	FAssetRegistryModule::AssetCreated(NewAsset);
}

void UTrajectoryExportOperation::ExportDataToAssets(const FTrajectoryExportContext& Context, FScopedSlowTask& Progress) const
{
	// Abort invalid asset was marked for processing.
	if (!AssetToProcess.IsValid() || !IsValid(AssetToProcess.Get()))
	{
		return;
	}
	
	// A valid skeleton is needed for preview information.
	USkeleton* Skeleton = TSoftObjectPtr<USkeleton>(Context.AssetInfo.Skeleton).LoadSynchronous();
	if (!IsValid(Skeleton) || Skeleton->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
	{
		return;
	}

	// A valid skeletal mesh is needed to query the ref pose.
	USkeletalMesh* SkeletalMesh = TSoftObjectPtr<USkeletalMesh>(Context.AssetInfo.SkeletalMesh).LoadSynchronous();
	if (!IsValid(SkeletalMesh) || SkeletalMesh->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
	{
		return;
	}

	// No bones in ref skeleton. Can't convert traced poses from component to local space.
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	if (RefSkeleton.GetRawBoneNum() == 0)
	{
		return;
	}
	
	// Inconsistent tracing of samples vs poses. Something wrong happened.
	const FGameplayTrajectory& RawTrajectory = *Context.Data;
	if (RawTrajectory.Samples.Num() != RawTrajectory.Poses.Num())
	{
		return;
	}

	// No data provided. Abort.
	if (RawTrajectory.Samples.IsEmpty())
	{
		return;
	}

	// Transform trajectory data to match export settings.
	FGameplayTrajectory FinalTrajectory = {};
	{
		if (Progress.ShouldCancel())
		{
			return;
		}
		
		Progress.EnterProgressFrame(1.f, LOCTEXT("ExportOperationProgress_RawTrajectory", "Transforming raw trajectory to match specified export settings..."));
		FTrajectoryToolsLibrary::TransformTrajectoryToMatchExportSettings(RawTrajectory, Context.Settings, FinalTrajectory);
	}

	// Properly configure newly animation sequence.
	{
		if (Progress.ShouldCancel())
		{
			return;
		}
		
		Progress.EnterProgressFrame(1.f, FText::Format(LOCTEXT("ExportOperationProgress_PreparingAsset", "Preparing asset: {0}"), FText::FromString(*AssetToProcess->GetName())));

		// @todo: store prev info (skeleton), so we can revert if overwriting asset.
		AssetToProcess->Modify(true);

		// Update skeleton info.
		{
			FScopedTransaction Transaction(LOCTEXT("UndoAction_ModifyAnimSequence", "Applying Skeleton to Animation Sequence(s)"));
			
			AssetToProcess->SetSkeleton(Skeleton);
			AssetToProcess->SetPreviewMesh(Skeleton->GetPreviewMesh());
			AssetToProcess->bEnableRootMotion = true;
		}
		
		const bool bShouldTransact = GeneratedAsset == nullptr;
		IAnimationDataController& Controller = AssetToProcess->GetController();
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("ExportTrajectoryConfigureAsset_Bracket", "Configure new animation sequence"), bShouldTransact);

		// Clean previous info, if any.
		Controller.InitializeModel();
		AssetToProcess->ResetAnimation();

		// Ensure anim sequence playback matches export settings.
		Controller.SetFrameRate(Context.Settings.FrameRate);
		Controller.SetNumberOfFrames(FinalTrajectory.Samples.Num() - 1);
		
		// Ensure UI displays proper framerate.
		AssetToProcess->ImportFileFramerate = static_cast<float>(Context.Settings.FrameRate.AsDecimal());
		AssetToProcess->ImportResampleFramerate = static_cast<int32>(Context.Settings.FrameRate.AsInterval());

		// Add all bone tracks in ref skeleton for now. @todo: Take into consideration bShouldExportOnlyAnimatedBones.
		const int32 NumBones = RefSkeleton.GetRefBonePose().Num();
		const TArray<FTransform>& RefBonePose = RefSkeleton.GetRefBonePose();
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
			Controller.AddBoneCurve(BoneName);
			Controller.SetBoneTrackKeys(BoneName, { FVector3f(RefBonePose[BoneIndex].GetTranslation()), FVector3f(RefBonePose[BoneIndex].GetTranslation()) }, { FQuat4f(RefBonePose[BoneIndex].GetRotation()), FQuat4f(RefBonePose[BoneIndex].GetRotation()) }, { FVector3f(RefBonePose[BoneIndex].GetScale3D()), FVector3f(RefBonePose[BoneIndex].GetScale3D()) });
		}
		
		Controller.NotifyPopulated();
	}

	// Buffer for final bone transforms to export.
	const TArray<TArray<FTransform>> & ComponentSpacePoses = FinalTrajectory.Poses;
	TArray<TArray<FTransform>> LocalSpacePoses = ComponentSpacePoses;
	
	// Convert all recorded poses to local space.
	{
		Progress.EnterProgressFrame(1.f, LOCTEXT("ExportOperationProgress_ConvertingToLocalSpace", "Converting trajectory data to local space..."));

		const bool bShouldTransact = GeneratedAsset == nullptr;
		IAnimationDataController& Controller = AssetToProcess->GetController();
		Controller.OpenBracket(LOCTEXT("ExportTrajectoryConvertPosesToLocalSpace_Bracket", "Convert recorded poses to local space"), bShouldTransact);

		// Ensure all poses are in local space before exporting.
		for (int32 PoseIndex = 0; PoseIndex  < ComponentSpacePoses.Num(); ++PoseIndex)
		{
			if (Progress.ShouldCancel())
			{
				Controller.CloseBracket(bShouldTransact);
				return;
			}
			
			const int32 NumBones = ComponentSpacePoses[PoseIndex].Num();
		
			// Convert from component space to local space.
			for (int32 BoneIndex = NumBones - 1; BoneIndex > 0; --BoneIndex)
			{
			    // @todo: We should probably warn about bone mismatch and cancel operation instead if any of the traced poses doesn't match the number of bones in the references skeleton.
				if (BoneIndex >= RefSkeleton.GetRefBonePose().Num())
				{
					continue;
				}
				
				const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					LocalSpacePoses[PoseIndex][BoneIndex] = LocalSpacePoses[PoseIndex][BoneIndex].GetRelativeTransform(LocalSpacePoses[PoseIndex][ParentIndex]);
				}
			}
		}

		Controller.CloseBracket(bShouldTransact);
	}

	// Output trajectory information to animation sequence.
	{
		Progress.EnterProgressFrame(1.f, FText::Format(LOCTEXT("ExportOperationProgress_OutputtingDataToAsset ", "Outputting trajectory data to asset: {0}"), FText::FromString(*AssetToProcess->GetName())));

		const bool bShouldTransact = GeneratedAsset == nullptr;
		IAnimationDataController& Controller = AssetToProcess->GetController();
		Controller.OpenBracket(LOCTEXT("ExportTrajectoryToAnimSequence_Bracket", "Export data to anim sequence"), bShouldTransact);
		
		// Root bone which to apply the trajectory transforms over time.
		const FName RootBoneName = RefSkeleton.GetBoneName(0);

		const int32 NumOfKeys = FinalTrajectory.Samples.Num();

		// Query all bone track available in anim sequence.
		TArray<FName> BoneTrackNames;
		Controller.GetModel()->GetBoneTrackNames(BoneTrackNames);
		
		for (int32 BoneTrackIndex = 0; BoneTrackIndex < BoneTrackNames.Num(); ++BoneTrackIndex)
		{
			if (Progress.ShouldCancel())
			{
				Controller.CloseBracket(bShouldTransact);
				return;
			}
			
			const FName & TrackName = BoneTrackNames[BoneTrackIndex];
			const bool bIsRootBone = RootBoneName == TrackName;

			// Holds all keys to export per bone step.
			FRawAnimSequenceTrack RawTrack;
			RawTrack.PosKeys.SetNum(NumOfKeys);
			RawTrack.RotKeys.SetNum(NumOfKeys);
			RawTrack.ScaleKeys.SetNum(NumOfKeys);
			
			for (int32 KeyIndex = 0; KeyIndex < NumOfKeys; KeyIndex++)
			{
				if (bIsRootBone)
				{
					const FQuat& DoubleQuat = FinalTrajectory.Samples[KeyIndex].Orientation;
					const FVector& DoubleVec = FinalTrajectory.Samples[KeyIndex].Position;
					
					RawTrack.PosKeys[KeyIndex] = FVector3f(DoubleVec.X, DoubleVec.Y, DoubleVec.Z);
					RawTrack.RotKeys[KeyIndex] = FQuat4f(DoubleQuat.X, DoubleQuat.Y, DoubleQuat.Z, DoubleQuat.W);
					RawTrack.ScaleKeys[KeyIndex] = FVector3f(1.0f, 1.0f, 1.0f);
				}
				else
				{
					const int32 TrackNameToBoneIndex = RefSkeleton.FindBoneIndex(TrackName);

					if (TrackNameToBoneIndex != INDEX_NONE && LocalSpacePoses[KeyIndex].IsValidIndex(TrackNameToBoneIndex))
					{
						RawTrack.PosKeys[KeyIndex].X = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetTranslation().X;
						RawTrack.PosKeys[KeyIndex].Y = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetTranslation().Y;
						RawTrack.PosKeys[KeyIndex].Z = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetTranslation().Z;
					
						RawTrack.RotKeys[KeyIndex].W = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetRotation().W;
						RawTrack.RotKeys[KeyIndex].X = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetRotation().X;
						RawTrack.RotKeys[KeyIndex].Y = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetRotation().Y;
						RawTrack.RotKeys[KeyIndex].Z = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetRotation().Z;
					
						RawTrack.ScaleKeys[KeyIndex].X = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetScale3D().X;
						RawTrack.ScaleKeys[KeyIndex].Y = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetScale3D().Y;
						RawTrack.ScaleKeys[KeyIndex].Z = LocalSpacePoses[KeyIndex][TrackNameToBoneIndex].GetScale3D().Z;	
					}
				}
			}

			// Output all traced keys for the current bone.
			Controller.SetBoneTrackKeys(TrackName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, bShouldTransact);
		}
		
		// Inform now that we are done changing the anim sequence.
		Controller.NotifyPopulated();

		Controller.CloseBracket(bShouldTransact);
	}
}


void UTrajectoryExportOperation::NotifyUserOfResults(const FTrajectoryExportContext& Context, FScopedSlowTask& Progress) const
{
	if (!AssetToProcess.IsValid() || !IsValid(AssetToProcess.Get()))
	{
		return;
	}

	// Create pop-up notification in editor UI
	constexpr float NotificationDuration = 5.0f;
	if (Progress.ShouldCancel())
	{
		Progress.EnterProgressFrame(1.f, FText(LOCTEXT("ExportOperationProgress_Cancelled", "Cancelled.")));
		
		// Notify user that retarget was cancelled
		FNotificationInfo Notification(FText::GetEmpty());
		Notification.ExpireDuration = NotificationDuration;
		Notification.Text = FText(LOCTEXT("CancelledExportOperation_NotificationTitle", "Export trajectory cancelled."));
		
		TSharedPtr<SNotificationItem> NotificationWidget = FSlateNotificationManager::Get().AddNotification(Notification);
		NotificationWidget->SetCompletionState(SNotificationItem::CS_Fail);
	}
	else
	{
		Progress.EnterProgressFrame(1.f, FText(LOCTEXT("ExportOperationProgress_Completed", "Export trajectory complete!")));
		
		// Log details of what assets were created
		if (GeneratedAsset.IsValid())
		{
			UE_LOGF(LogTemp, Display, "Export trajectory - New Asset Created: %ls", *GeneratedAsset->GetName());
		}
		else
		{
			UE_LOGF(LogTemp, Display, "Export trajectory - Asset Modified: %ls", *AssetToProcess->GetName());
		}
		
		// Notify user that export was completed
		FNotificationInfo Notification(FText::GetEmpty());
		Notification.ExpireDuration = NotificationDuration;
		Notification.ExpireDuration = 10.f;
		Notification.Hyperlink = FSimpleDelegate::CreateLambda([Asset = GeneratedAsset]()
		{
			// Select all new assets and show them in the content browser.
			const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			TArray<FAssetData> NewAssetsData = { Asset.Get() };
			ContentBrowserModule.Get().SyncBrowserToAssets(NewAssetsData);
		});

		Notification.HyperlinkText = LOCTEXT("OpenGeneratedAssetFromTrajectory", "Browse to asset");
		Notification.Text = FText::Format(
			LOCTEXT("CompletedExportingTrajectoryNotification", "{0}'s trajectory data was exported to {1} animation sequence."),
			FText::FromString(Context.SourceObjectName),
			FText::FromString(*AssetToProcess->GetName()));
		
		TSharedPtr<SNotificationItem> NotificationWidget = FSlateNotificationManager::Get().AddNotification(Notification);
		NotificationWidget->SetCompletionState(SNotificationItem::CS_Success);
	}	
}

void UTrajectoryExportOperation::CleanupIfCancelled(const FScopedSlowTask& Progress) const
{
	if (!Progress.ShouldCancel())
	{
		return;
	}

	// Revert any changes.
	if (ActiveTransaction.IsValid())
	{
		// We need to call Apply on the global undo, or cancelling the transaction doesn't actually roll back.
		GUndo->Apply();
		ActiveTransaction->Cancel();
	}

	// Any generated assets we just delete them since their changes were not transacted.
	if (GeneratedAsset.IsValid())
	{
		// Notify the asset registry
		FAssetRegistryModule::AssetDeleted(GeneratedAsset.Get());
	
		// Rename the asset we created out of the way
		GeneratedAsset->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
		
		GeneratedAsset->MarkAsGarbage();
		ObjectTools::DeleteAssets({ GeneratedAsset.Get() }, false);	
	}
}

#undef LOCTEXT_NAMESPACE
