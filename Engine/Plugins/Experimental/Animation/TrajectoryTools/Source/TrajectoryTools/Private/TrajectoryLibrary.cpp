// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrajectoryLibrary.h"

#include "Animation/AnimSequence.h"
#include "UObject/Object.h"
#include "Algo/BinarySearch.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "TrajectoryExportOperation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TrajectoryLibrary)

#define LOCTEXT_NAMESPACE "TrajectoryLibrary"

void FTrajectoryToolsLibrary::GetRangeOverlaps(const FGameplayTrajectory& InTrajectory, const TRange<int32>& InSampleRange, FRangeOverlapTestResult& InOutOverlapResult)
{
	int OverlapCount = 0;
	
	if (!InTrajectory.TraceInfo.IsEmpty() && !InTrajectory.Samples.IsEmpty() && !InSampleRange.IsEmpty() && !InSampleRange.IsDegenerate())
	{
		for (int i = 0; i < InTrajectory.TraceInfo.Num(); ++i)
		{
			if (bool bOverlaps =  InTrajectory.TraceInfo.Ranges[i].Overlaps(InSampleRange))
			{
				InOutOverlapResult.bOverlaps = bOverlaps;
				InOutOverlapResult.Ranges.Push(i);
				++OverlapCount;
			}
		}
	}

	ensureMsgf(OverlapCount <= 2, TEXT("This function should only be called where we know only two range are possibly overlapping"));
}

void FTrajectoryToolsLibrary::GetSampleIndicesForMatchedSampleTime(const FGameplayTrajectory& InTrajectory, double InRequestedSampleTime, TPair<int32, int32>& OutMatchedSampleIndicesRange)
{
	if (!InTrajectory.Samples.IsEmpty())
	{
		InRequestedSampleTime = FMath::Clamp(InRequestedSampleTime, 0, InTrajectory.Samples.Last().Time);
		
		const int32 EndSampleIndex = Algo::LowerBoundBy(InTrajectory.Samples, InRequestedSampleTime, &FGameplayTrajectory::FSample::Time);
		check(EndSampleIndex < InTrajectory.Samples.Num())
		const int32 StartSampleIndex = FMath::Clamp(EndSampleIndex - 1, 0, InTrajectory.Samples.Num() - 1);
		
		OutMatchedSampleIndicesRange.Get<0>() = StartSampleIndex;
		OutMatchedSampleIndicesRange.Get<1>() = EndSampleIndex;
	}
	else
	{
		OutMatchedSampleIndicesRange.Get<0>() = INDEX_NONE;
		OutMatchedSampleIndicesRange.Get<1>() = INDEX_NONE;
	}
}

bool FTrajectoryToolsLibrary::GetTransformAtTimeInTrajectory(const FGameplayTrajectory& InTrajectory, double InRequestedTime, FTransform& OuTransform)
{
	// Get keys for smallest range that contains the requested time.
	TPair<int32, int32> IndicesRangeThatMatchRequestedTime = { INDEX_NONE, INDEX_NONE };
	GetSampleIndicesForMatchedSampleTime(InTrajectory, InRequestedTime, IndicesRangeThatMatchRequestedTime);

	// Invalid indices.
	if (IndicesRangeThatMatchRequestedTime.Get<0>() == INDEX_NONE || IndicesRangeThatMatchRequestedTime.Get<1>() == INDEX_NONE)
	{
		return false;
	}

	// Time requested exactly matches a sample (The case of the calling GetSampleIndicesForMatchedSampleTime() with time 0 or having trajectory with one sample).
	if (IndicesRangeThatMatchRequestedTime.Get<0>() == IndicesRangeThatMatchRequestedTime.Get<1>())
	{
		const FGameplayTrajectory::FSample& Sample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<0>()];

		OuTransform = FTransform(Sample.Orientation, Sample.Position);
	}
	// Interpolate keys to exactly match the requested time.
	else
	{
		const FGameplayTrajectory::FSample& StartSample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<0>()];
		const FGameplayTrajectory::FSample& EndSample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<1>()];
		
		const double Alpha = (InRequestedTime - StartSample.Time) / (EndSample.Time - StartSample.Time);

		OuTransform = FTransform(FQuat::Slerp(StartSample.Orientation, EndSample.Orientation, Alpha), FMath::Lerp(StartSample.Position, EndSample.Position, Alpha));
	}

	return true;
}

bool FTrajectoryToolsLibrary::GetPoseAtTimeInTrajectory(const FGameplayTrajectory& InTrajectory, double InRequestedTime, TArray<FTransform>& OutCSPose)
{
	// Get keys for smallest range that contains the requested time.
	TPair<int32, int32> IndicesRangeThatMatchRequestedTime = { INDEX_NONE, INDEX_NONE };
	GetSampleIndicesForMatchedSampleTime(InTrajectory, InRequestedTime, IndicesRangeThatMatchRequestedTime);

	// Invalid indices.
	if (IndicesRangeThatMatchRequestedTime.Get<0>() == INDEX_NONE || IndicesRangeThatMatchRequestedTime.Get<1>() == INDEX_NONE)
	{
		return false;
	}
	
	// OutCSPose.Reserve(InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<0>()].Num());
	OutCSPose.SetNumZeroed(InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<0>()].Num());
	
	// Time requested exactly matches a sample (The case of the calling GetSampleIndicesForMatchedSampleTime() with time 0 or having trajectory with one sample).
	if (IndicesRangeThatMatchRequestedTime.Get<0>() == IndicesRangeThatMatchRequestedTime.Get<1>())
	{
		const TArray<FTransform>& Pose = InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<0>()];

		for (int i = 0; i < Pose.Num(); ++i)
		{
			OutCSPose[i] = Pose[i];
		}
	}
	// Interpolate keys to exactly match the requested time.
	else
	{
		FRangeOverlapTestResult TestResult{};
		GetRangeOverlaps(InTrajectory, TRange<int32>(IndicesRangeThatMatchRequestedTime.Key, IndicesRangeThatMatchRequestedTime.Value), TestResult);

		if (TestResult.bOverlaps && TestResult.Ranges.Num() == 2)
		{
			const FGameplayTrajectory::FSample& StartSample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<0>()];
			const FGameplayTrajectory::FSample& EndSample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<1>()];

			const TArray<FTransform>& StartPose = InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<0>()];
			const TArray<FTransform>& EndPose = InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<1>()];

			if (FMath::Abs(StartSample.Time - InRequestedTime) < FMath::Abs(EndSample.Time - InRequestedTime))
			{
				OutCSPose = StartPose;
			}
			else
			{
				OutCSPose = EndPose;
			}
			
			return true;
		}
		
		const TArray<FTransform>& StartPose = InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<0>()];
		const TArray<FTransform>& EndPose = InTrajectory.Poses[IndicesRangeThatMatchRequestedTime.Get<1>()];

		// check(StartPose.Num() == EndPose.Num() && EndPose.Num() == OutCSPose.Num());
		
		const FGameplayTrajectory::FSample& StartSample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<0>()];
		const FGameplayTrajectory::FSample& EndSample = InTrajectory.Samples[IndicesRangeThatMatchRequestedTime.Get<1>()];
		
		const double Alpha = (InRequestedTime - StartSample.Time) / (EndSample.Time - StartSample.Time);

		for (int i = 0; i < StartPose.Num(); ++i)
		{
			OutCSPose[i].Blend(StartPose[i], EndPose[i], Alpha);
		}
	}

	return true;
}

void FTrajectoryToolsLibrary::TransformTrajectoryToMatchFrameRate(const FGameplayTrajectory& InTrajectory, FFrameRate InFrameRate, FGameplayTrajectory& OutFrameMatchedTrajectory)
{
	check(InTrajectory.Samples.Num() == InTrajectory.Poses.Num());
	
	if (!InTrajectory.Samples.IsEmpty())
	{
		if (InTrajectory.Samples.Num() == 1)
		{
			// Fast-path when only one sample is present.
			OutFrameMatchedTrajectory.Samples = InTrajectory.Samples;
			OutFrameMatchedTrajectory.Poses = InTrajectory.Poses;
			OutFrameMatchedTrajectory.TraceInfo = InTrajectory.TraceInfo;
		}
		else
		{
			const double TrajectoryPlayLength = InTrajectory.Samples.Num() == 1 ? InTrajectory.Samples[0].Time : InTrajectory.Samples.Last().Time - InTrajectory.Samples[0].Time;
			const double TrajectorySampleRate = InFrameRate.AsInterval();
			const int32 TotalSamples = InFrameRate.AsFrameTime(TrajectoryPlayLength).CeilToFrame().Value;

			checkf(TotalSamples > 0, TEXT("Total output of samples is zero or less."))

			// Clear buffer before processing
			OutFrameMatchedTrajectory.Samples.Reset(TotalSamples);
			OutFrameMatchedTrajectory.Poses.Reset(TotalSamples);
			OutFrameMatchedTrajectory.TraceInfo.Reset(InTrajectory.TraceInfo.Num());

			// Trace ranges to match that of the source trajectory.
			for (int32 i = 0; i < InTrajectory.TraceInfo.Num(); ++i)
			{
				const int LowerBound = InTrajectory.TraceInfo.Ranges[i].GetLowerBoundValue();
				const int UpperBound = InTrajectory.TraceInfo.Ranges[i].GetUpperBoundValue();

				const double LowerBoundSampleTime = InTrajectory.Samples[LowerBound].Time;
				const double UpperBoundSampleTime = InTrajectory.Samples[UpperBound].Time;
			
				const int FrameMatchedLowerBoundIndex = FMath::Clamp(InFrameRate.AsFrameTime(LowerBoundSampleTime).FloorToFrame().Value, 0, TotalSamples - 1);
				const int FrameMatchedUpperBoundIndex = FMath::Clamp(InFrameRate.AsFrameTime(UpperBoundSampleTime).FloorToFrame().Value, 0, TotalSamples - 1);

				OutFrameMatchedTrajectory.TraceInfo.Ranges.Emplace(TRangeBound<int32>::Inclusive(FrameMatchedLowerBoundIndex), TRangeBound<int32>::Inclusive(FrameMatchedUpperBoundIndex));
				OutFrameMatchedTrajectory.TraceInfo.SkeletalMeshInfos.Push(InTrajectory.TraceInfo.SkeletalMeshInfos[i]);
			}
		
			for (int32 OutSampleIndex = 0; OutSampleIndex < TotalSamples; ++OutSampleIndex)
			{
				const double RequestedSampleTime = FMath::Clamp(static_cast<double>(OutSampleIndex) * TrajectorySampleRate, 0.0f, TrajectoryPlayLength);

				FTransform SampleTransform;
				check(GetTransformAtTimeInTrajectory(InTrajectory, RequestedSampleTime, SampleTransform))

				OutFrameMatchedTrajectory.Samples.AddDefaulted();
				FGameplayTrajectory::FSample& TimeMatchedSample = OutFrameMatchedTrajectory.Samples.Last();
	
				TimeMatchedSample.Position = SampleTransform.GetLocation();
				TimeMatchedSample.Orientation = SampleTransform.GetRotation();
				TimeMatchedSample.Time = RequestedSampleTime;

				OutFrameMatchedTrajectory.Poses.AddDefaulted();
				TArray<FTransform>& TimeMatchedPose = OutFrameMatchedTrajectory.Poses.Last();

				check(GetPoseAtTimeInTrajectory(InTrajectory, RequestedSampleTime, TimeMatchedPose));
			}
		}
	}
	else
	{
		// @todo: Warning.
	}
}

void FTrajectoryToolsLibrary::TransformTrajectoryToMatchExportSettings(const FGameplayTrajectory& InTrajectory, const FTrajectoryExportSettings& InExportSettings, FGameplayTrajectory& OutTransformedTrajectory)
{
	if (InTrajectory.Samples.IsEmpty())
	{
		// error trajectory empty
		return;
	}

	if (!InExportSettings.IsValid())
	{
		// invalid export settings
		return;
	}

	// Ensure samples match framerate
	TransformTrajectoryToMatchFrameRate(InTrajectory, InExportSettings.FrameRate, OutTransformedTrajectory);
	
	// Make all samples be relative to the position at sample given by the origin time.
	if (InExportSettings.bShouldForceOrigin)
	{
		const int SampleIndexAtOriginTime = InExportSettings.FrameRate.AsFrameTime(InExportSettings.OriginTime).FrameNumber.Value;
		const FVector OffsetToOrigin = -OutTransformedTrajectory.Samples[SampleIndexAtOriginTime].Position;
		
		for (FGameplayTrajectory::FSample& Sample : OutTransformedTrajectory.Samples)
		{
			Sample.Position += OffsetToOrigin;
		}
	}
	
	// Prune samples not within range.
	{
		int StartRangeSampleIndex = INDEX_NONE;
		int EndRangeSampleIndex = INDEX_NONE;
		
		// @todo: Improve with two pointer technique.
		for (int SampleIndex = 0; SampleIndex <  OutTransformedTrajectory.Samples.Num(); ++SampleIndex)
		{
			FGameplayTrajectory::FSample& Sample = OutTransformedTrajectory.Samples[SampleIndex];

			// Find lower bound index
			if (StartRangeSampleIndex == INDEX_NONE)
			{
				const bool bPassesLowerBoundCheck = FMath::IsNearlyEqual(Sample.Time, InExportSettings.Range.Min) || Sample.Time > InExportSettings.Range.Min;

				if (bPassesLowerBoundCheck)
				{
					StartRangeSampleIndex = SampleIndex;
				}
			}

			// Find upper bound index
			if (EndRangeSampleIndex == INDEX_NONE)
			{
				const bool bPassesUpperBoundCheck = FMath::IsNearlyEqual(Sample.Time, InExportSettings.Range.Max) || Sample.Time > InExportSettings.Range.Max;

				if (bPassesUpperBoundCheck)
				{
					EndRangeSampleIndex = SampleIndex;
				}
			}
		}

		// We never found the end, but we have a start. Default end to last sample.
		if (StartRangeSampleIndex != INDEX_NONE && EndRangeSampleIndex == INDEX_NONE)
		{
			EndRangeSampleIndex = OutTransformedTrajectory.Samples.Num() - 1;	
		}
		
		// Update time
		for (FGameplayTrajectory::FSample& Sample : OutTransformedTrajectory.Samples)
		{
			Sample.Time -= InExportSettings.Range.Min;
		}

		// @todo: Improve the efficiency of this by not copying so much.
		
		// Output pruned trajectory
		const int TotalSamplesAfterPrune = EndRangeSampleIndex - StartRangeSampleIndex + 1;
		const TArray<FGameplayTrajectory::FSample> PrunedTrajectory = TArray<FGameplayTrajectory::FSample>(MakeArrayView(OutTransformedTrajectory.Samples.GetData() + StartRangeSampleIndex, TotalSamplesAfterPrune));
		const TArray<TArray<FTransform>> PrunedPoses = TArray<TArray<FTransform>>(MakeArrayView(OutTransformedTrajectory.Poses.GetData() + StartRangeSampleIndex, TotalSamplesAfterPrune));
		
		OutTransformedTrajectory.Samples.Reset();
		OutTransformedTrajectory.Samples.Append(PrunedTrajectory);
		OutTransformedTrajectory.Poses.Reset();
		OutTransformedTrajectory.Poses.Append(PrunedPoses);
	}
}

#undef LOCTEXT_NAMESPACE
