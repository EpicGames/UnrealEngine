// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrajectoryRewindDebuggerExtension.h"

#include "Common/ProviderLock.h"
#include "ToolMenus.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "Debug/DebugDrawService.h"
#include "DrawDebugHelpers.h"
#include "IRewindDebuggerModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Canvas.h"
#include "SceneView.h"
#include "SExportTrajectoriesWindow.h"
#include "Engine/Engine.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"
#include "Animation/AnimSequence.h"
#include "Widgets/Input/SButton.h"
#include "Logging/StructuredLog.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerTrajectory"

FRewindDebuggerTrajectory::~FRewindDebuggerTrajectory()
{
	if (UToolMenus* Menus = UToolMenus::Get())
	{
		if (const IRewindDebuggerModule* RewindDebuggerModule = FModuleManager::GetModulePtr<IRewindDebuggerModule>("RewindDebugger"))
		{
			if (UToolMenu* Menu = Menus->FindMenu(RewindDebuggerModule->GetMainToolbarName()))
			{
				Menu->RemoveSection("TrajectoryWorkflows");
			}
			
			if (UToolMenu* PreviewMenu = Menus->FindMenu(RewindDebuggerModule->GetPreviewMenuName()))
			{
				PreviewMenu->RemoveSection("PreviewTrajectories");
			}
		}
	}
}

FRewindDebuggerTrajectory::FRewindDebuggerTrajectory()
{
}

void FRewindDebuggerTrajectory::Initialize()
{
	// Generate widgets.
	GenerateMenu();

	// Colors for trajectories. (Up to 4 rn, should cycle if trying to index out of bounds).
	DebugDrawColors.Push(FColor::Cyan);
	DebugDrawColors.Push(FColor::Emerald);
	DebugDrawColors.Push(FColor::Magenta);
	DebugDrawColors.Push(FColor::Yellow);
}

void FRewindDebuggerTrajectory::Shutdown()
{
	// Delete preview window.
	if (BakeOutWindow.IsValid())
	{
		FSlateApplication::Get().DestroyWindowImmediately(BakeOutWindow.ToSharedRef());
		BakeOutWindow = nullptr;
	}
}

void FRewindDebuggerTrajectory::Reset()
{
	// Delete preview window. @todo: This is temporary.
	if (BakeOutWindow.IsValid())
	{
		FSlateApplication::Get().DestroyWindowImmediately(BakeOutWindow.ToSharedRef());
		BakeOutWindow = nullptr;
	}
	
	// No valid world to visualize.
	WorldToVisualize = nullptr;

	// Stop hooking into debug draw.
	if (DebugDrawDelegateHandle.IsValid())
	{
		UDebugDrawService::Unregister(DebugDrawDelegateHandle);
		DebugDrawDelegateHandle.Reset();
	}

	// Clear state.
	State.Reset();
}

void FRewindDebuggerTrajectory::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	check(RewindDebugger);

	// Early out.
	if (RewindDebugger->IsPIESimulating() || RewindDebugger->GetRecordingDuration() == 0)
	{
		Reset();

		return;
	}

	// @todo: Should we trace actor transforms?
	// @todo: Use orientation from trajectory vs captured pose.

	// Ensure we use RewindDebugger's world for debug drawing.
	WorldToVisualize = RewindDebugger->GetWorldToVisualize();
	
	// Hook into engine flag for debug drawing.
	EnsureDebugDrawDelegateExists();
	
	check(State.Trajectories.Num() == State.ObjectInfos.Num());
}

void FRewindDebuggerTrajectory::AnalysisSessionOpened(IRewindDebugger* RewindDebugger)
{
	// Clear up any cached state / variables.
	Reset();
}

void FRewindDebuggerTrajectory::AnalysisSessionClosed(IRewindDebugger* RewindDebugger)
{
	
}

void FRewindDebuggerTrajectory::Clear(IRewindDebugger* RewindDebugger)
{
	// Clear up any cached state / variables.
	Reset();
}

void FRewindDebuggerTrajectory::UpdateState(IRewindDebugger* RewindDebugger)
{
	// Query info from trace file.
	const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();
	if (Session)
	{
		const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
		const IAnimationProvider* AnimationProvider = Session->ReadProvider<IAnimationProvider>("AnimationProvider");

		if (GameplayProvider && AnimationProvider)
		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

			// Note that recording index is always zero since RewindDebugger starts a new trace for every recording.
			if (const IGameplayProvider::RecordingInfoTimeline* Recording = GameplayProvider->GetRecordingInfo(0))
			{
				const uint64 EventCount = Recording->GetEventCount();

				if (EventCount > 0)
				{
					const FRecordingInfoMessage& LastEvent = Recording->GetEvent(EventCount - 1);
					
					State.Extract.TraceStarTime = 0;
					State.Extract.TraceEndTime = LastEvent.ProfileTime;
				}
				else
				{
					// Empty trace session. Abort.
					Reset();
					return;
				}
			}
			
			BuildTrajectoryOwnersList(RewindDebugger, GameplayProvider, State.Extract.TraceStarTime, State.Extract.TraceEndTime, State.ObjectInfos);
			{
				TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
				BuildTrajectorySkeletalMeshInfoList(GameplayProvider, AnimationProvider, State.Extract.TraceStarTime, State.Extract.TraceEndTime, State.ObjectInfos, State.SkelMeshInfos);
				BuildTrajectories(GameplayProvider, AnimationProvider, State.Extract.TraceStarTime, State.Extract.TraceEndTime, State.ObjectInfos, State.Trajectories);
			}
			UpdateDebugInfos(GameplayProvider, State.ObjectInfos, State.DebugInfos);
		}
	}
	else
	{
		// Invalid trace session. Abort.
		Reset();
	}
}

void FRewindDebuggerTrajectory::GenerateMenu()
{
	const IRewindDebuggerModule* RewindDebuggerModule = FModuleManager::GetModulePtr<IRewindDebuggerModule>("RewindDebugger");
	if (!ensure(RewindDebuggerModule))
	{
		return;
	}

	// Export button lives in the main toolbar's TrajectoryWorkflows section
	if (UToolMenu* Menu = UToolMenus::Get()->FindMenu(RewindDebuggerModule->GetMainToolbarName()))
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("TrajectoryWorkflows", LOCTEXT("Trajectory Tools", "Trajectory Tools"));
		Section.AddSeparator("TrajectoryWorkflows");

		static const FName ExportButtonName = "ExportToolbarMenu";
		static const FText ExportLabel = LOCTEXT("ExportTrajectoriesLabel", "Export");
		static const FText ExportTooltip = LOCTEXT("ExportTrajectoriesTooltip", "Export recording/trajectory into a standalone asset");

		const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		TSharedRef<SWidget> CustomButton = SNew(SButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.ToolTipText(ExportTooltip)
			.IsEnabled_Lambda([]()
				{
					IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
					return RewindDebugger ? !RewindDebugger->IsPIESimulating() || !RewindDebugger->IsRecording() : false;
				})
			.OnClicked_Lambda([this]()
				{
					ShowBakeOutTrajectoryWindow();
					return FReply::Handled();
				})
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 6.f, 0.f)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Toolbar.Export"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(ExportLabel)
					.TextStyle(&ToolBarStyle.LabelStyle)
				]
			];

		Section.AddEntry(FToolMenuEntry::InitWidget(ExportButtonName, CustomButton, ExportLabel, false, true, false, ExportTooltip));
	}

	// Trajectories preview lives under Preview → Trajectories in the named preview menu
	{
		UToolMenu* PreviewMenu = UToolMenus::Get()->ExtendMenu(RewindDebuggerModule->GetPreviewMenuName());
		FToolMenuSection& Section = PreviewMenu->FindOrAddSection("PreviewTrajectories", LOCTEXT("OverlaysSectionLabel", "Overlays"));
		Section.AddSubMenu(
			"Trajectories",
			LOCTEXT("ToggleDebugDrawTrajectoriesLabel", "Trajectories"),
			LOCTEXT("ToggleDebugDrawTrajectoriesTooltip", "Toggle recorded trajectories to display in viewport"),
			FNewToolMenuDelegate::CreateRaw(this, &FRewindDebuggerTrajectory::MakeToggleDebugDrawMenu),
			false,
			FSlateIcon(),
			false
		);
	}
}

void FRewindDebuggerTrajectory::MakeToggleDebugDrawMenu(UToolMenu* InMenu)
{
	// Prepare trajectories to preview.
	if (IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		UpdateState(RewindDebugger);
	}
	
	FToolMenuSection& Section = InMenu->FindOrAddSection("ToggleDebugDrawOptions");

	for (int i = 0; i < State.Trajectories.Num(); ++i)
	{
		FExtensionState::FDebugInfo& DebugInfoEntry = State.DebugInfos[i];
		
		FUIAction UIAction;
		{
			// @todo: For safety, this should probably just copy the component id and try to find it, then toggle it.
			UIAction.ExecuteAction = FExecuteAction::CreateLambda([&DebugInfoEntry](){ DebugInfoEntry.bShouldDraw = !DebugInfoEntry.bShouldDraw; });
			UIAction.GetActionCheckState = FGetActionCheckState::CreateLambda([&DebugInfoEntry] { return DebugInfoEntry.bShouldDraw ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; });
			UIAction.CanExecuteAction = FCanExecuteAction::CreateLambda([this, i]()
				{
					const bool bHasAnyData = !State.Trajectories[i].Samples.IsEmpty() && !State.Trajectories[i].Poses.IsEmpty();
					return bHasAnyData;
				});
		}

		Section.AddMenuEntry(DebugInfoEntry.Name, FText::FromName(DebugInfoEntry.Name), FText::FromName(DebugInfoEntry.Name), FSlateIconFinder::FindIconForClass(USkeletalMeshComponent::StaticClass(), TEXT("SCS.Component")), UIAction, EUserInterfaceActionType::Check);
	}
}

void FRewindDebuggerTrajectory::EnsureDebugDrawDelegateExists()
{
	if (!DebugDrawDelegateHandle.IsValid())
	{
		DebugDrawDelegateHandle = UDebugDrawService::Register(TEXT("GameplayDebug"), FDebugDrawDelegate::CreateRaw(this, &FRewindDebuggerTrajectory::DebugDraw));
	}
}

void FRewindDebuggerTrajectory::DebugDraw(UCanvas* InCanvas, APlayerController* InController)
{
	// @todo: Only draw object ids which are part of the target object.
	if (WorldToVisualize.IsValid() && !State.Trajectories.IsEmpty())
	{
		for (int TrajectoryIndex = 0; TrajectoryIndex < State.Trajectories.Num(); ++TrajectoryIndex)
		{
			if (State.DebugInfos[TrajectoryIndex].bShouldDraw)
			{
				const FGameplayTrajectory& DebugTrajectoryPair = State.Trajectories[TrajectoryIndex];
				const FColor& DebugDrawColorForTrajectory = DebugDrawColors[TrajectoryIndex % DebugDrawColors.Num()];
			
				for (int SampleIndex = 0; SampleIndex < DebugTrajectoryPair.Samples.Num(); ++SampleIndex)
				{
					const FGameplayTrajectory::FSample& Sample = DebugTrajectoryPair.Samples[SampleIndex];
					
					// Draw trajectory
					if (SampleIndex + 1 < DebugTrajectoryPair.Samples.Num())
					{
						const FGameplayTrajectory::FSample& FutureSample = DebugTrajectoryPair.Samples[SampleIndex + 1];

						DrawDebugLine(WorldToVisualize.Get(), Sample.Position, FutureSample.Position, DebugDrawColorForTrajectory);
					}

					// Draw sections at every second
					if (SampleIndex % 20 == 0)
					{
						DrawDebugPoint(WorldToVisualize.Get(), Sample.Position, 5, DebugDrawColorForTrajectory);
					}
				}
			}
		}
	}
}

void FRewindDebuggerTrajectory::ShowBakeOutTrajectoryWindow()
{
	if (BakeOutWindow.IsValid())
	{
		FSlateApplication::Get().DestroyWindowImmediately(BakeOutWindow.ToSharedRef());
	}

	// Prepare data before opening window.
	if (IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		UpdateState(RewindDebugger);
	}
	
	// Custom window to setup bake out settings and execute the action
	SAssignNew(BakeOutWindow, SWindow)
	.Title(LOCTEXT("BakeOutTrajectoryWindowTitle", "Export Recording / Trajectories"))
	.SizingRule(ESizingRule::UserSized)
	.SupportsMaximize(true)
	.SupportsMinimize(true)
	.MinWidth(850)
	.MinHeight(500)
	.CreateTitleBar(true)
	.HasCloseButton(true)
	.ClientSize(FVector2f{1280, 720});

	// Gather all the associated owner names
	TArray<FName> OwnerNames;
	for (const FExtensionState::FDebugInfo& DebugInfo : State.DebugInfos)
	{
		OwnerNames.Add(DebugInfo.Name);
	}

	// Init custom trajectory export window
	TSharedPtr<SExportTrajectoriesWindow> ExportWindow;
	BakeOutWindow->SetContent
	(
		SAssignNew(ExportWindow, SExportTrajectoriesWindow)
		.Trajectories(State.Trajectories)
		.WidgetWindow(BakeOutWindow)
		.OwnerNames(OwnerNames)
		.DebugInfos(State.DebugInfos)
		.SkelMeshInfos(State.SkelMeshInfos)
	);
	BakeOutWindow->SetOnWindowClosed(FOnWindowClosed::CreateLambda([this](const TSharedRef<SWindow>&){ BakeOutWindow = nullptr; }));

	// Hook into main window as a modal window.
	/*TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		const IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}
	ExportWindow->SetCanTick(true);*/
	
	// We use a modal window to ensure that we can store id/values from rewind debugger messages and ensure that its data provider are valid,
	// otherwise we would have to copy all the information when opening the window.
	// FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
	FSlateApplication::Get().AddWindow(BakeOutWindow.ToSharedRef());
}

void FRewindDebuggerTrajectory::BuildTrajectoryOwnersList(const IRewindDebugger* InRewindDebugger, const IGameplayProvider* InGameplayProvider, double InTraceStartTime, double InTraceEndTime, TArray<FObjectInfo> & OutObjectInfos)
{
	// Clear up previous info.
	OutObjectInfos.Reset();
	
	// Used to filter which objects we can use to build our trajectory.
	const FClassInfo* SkelMeshCmpClassInfo = InGameplayProvider->FindClassInfo(*USkeletalMeshComponent::StaticClass()->GetPathName());
	
	// Not a single actor with a skeletal mesh component was traced, safe to early out.
	if (SkelMeshCmpClassInfo == nullptr)
	{
		return;
	}
	
	// Query target actor of the trace file / session.
	InGameplayProvider->EnumerateObjects(InTraceStartTime, InTraceEndTime, [&OutObjectInfos,InGameplayProvider, SkelMeshCmpClassInfo, InRewindDebugger](const FObjectInfo& InObjectInfo)
	{
		// For all root objects find their skeletal mesh component's id, if any.
		if (!InObjectInfo.GetOuterId().IsSet())
		{
			InGameplayProvider->EnumerateSubobjects(InObjectInfo.GetId(), [&OutObjectInfos, InGameplayProvider, SkelMeshCmpClassInfo](const RewindDebugger::FObjectId& InSubObjectId)
			{
				const bool bIsSubObjectValid = InSubObjectId.IsSet();
				const FObjectInfo& SubObjectInfo = InGameplayProvider->GetObjectInfo(InSubObjectId);

				const bool bIsSubObjectASkelMesh = SubObjectInfo.ClassId == SkelMeshCmpClassInfo->Id || InGameplayProvider->IsSubClassOf(SubObjectInfo.ClassId, SkelMeshCmpClassInfo->Id);
				if (bIsSubObjectValid && bIsSubObjectASkelMesh)
				{
					OutObjectInfos.Push(SubObjectInfo);
				}
			});
		}
		else
		{
			const bool bIsObjectValid = InObjectInfo.GetId().IsSet();
			const bool bIsObjectASkelMesh = InObjectInfo.ClassId == SkelMeshCmpClassInfo->Id || InGameplayProvider->IsSubClassOf(InObjectInfo.ClassId, SkelMeshCmpClassInfo->Id);
			if (bIsObjectValid && bIsObjectASkelMesh)
			{
				const bool bHasOwningActor = InRewindDebugger->FindTypedOuterInfo<AActor>(InGameplayProvider, InObjectInfo.GetUObjectId()) != nullptr;
				if (bHasOwningActor)
				{
					OutObjectInfos.Push(InObjectInfo);
				}
			}
		}
	});
}

void FRewindDebuggerTrajectory::BuildTrajectorySkeletalMeshInfoList(const IGameplayProvider* InGameplayProvider, const IAnimationProvider* InAnimationProvider, double InTraceStartTime, double InTraceEndTime, const TArray<FObjectInfo>& InSkeletalMeshComponentObjectInfos, TArray<FSkeletalMeshInfo>& OutSkeletalMeshInfos)
{
	// Clear up previous info.
	OutSkeletalMeshInfos.Reset();
	
	for (int SkelMeshCmpIndex = 0; SkelMeshCmpIndex < InSkeletalMeshComponentObjectInfos.Num(); ++SkelMeshCmpIndex)
	{
		const FObjectInfo& SkeletalMeshComponentInfo = InSkeletalMeshComponentObjectInfos[SkelMeshCmpIndex];
		
		InAnimationProvider->ReadSkeletalMeshPoseTimeline(SkeletalMeshComponentInfo.GetUObjectId(), [InAnimationProvider, InGameplayProvider, InTraceStartTime, InTraceEndTime, SkelMeshCmpIndex, &SkeletalMeshComponentInfo, &OutSkeletalMeshInfos](const IAnimationProvider::SkeletalMeshPoseTimeline& TimelineData, bool bHasCurves)
		{
			TimelineData.EnumerateEvents(InTraceStartTime, InTraceEndTime, [InAnimationProvider, InGameplayProvider, SkelMeshCmpIndex, &SkeletalMeshComponentInfo, &OutSkeletalMeshInfos](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InPoseMessage) -> TraceServices::EEventEnumerate
			{
				const FSkeletalMeshInfo* SkeletalMeshInfo = InAnimationProvider->FindSkeletalMeshInfo(InPoseMessage.MeshId);
				const FObjectInfo* SkeletalMeshObjectInfo = InGameplayProvider->FindObjectInfo(InPoseMessage.MeshId); // @todo: Check if this search can be remove and just use the already available id.

				// Query skeletal mesh information.
				if (!SkeletalMeshInfo || !SkeletalMeshObjectInfo)
				{
					OutSkeletalMeshInfos.Reset();
					UE_LOGFMT(LogTemp, Warning, "No SkelMeshInfo or SkelMeshObjectInfo provided. For object: {name}", *SkeletalMeshComponentInfo.Name);
					return TraceServices::EEventEnumerate::Stop;
				}
				
				if (!OutSkeletalMeshInfos.IsValidIndex(SkelMeshCmpIndex))
				{
					OutSkeletalMeshInfos.Push(*SkeletalMeshInfo);
					// UE_LOGFMT(LogTemp, Warning, "[{0}] - Skel Mesh Info START - \nStarting:{1}", SkelMeshCmpIndex, SkeletalMeshObjectInfo->PathName);

					const FObjectInfo* SkeletonObjectInfo = InGameplayProvider->FindObjectInfo(SkeletalMeshInfo->SkeletonId);
					// UE_LOGFMT(LogTemp, Warning, "[{0}] - Skeleton Info START - \nStarting:{1}", SkelMeshCmpIndex, SkeletonObjectInfo != nullptr? SkeletonObjectInfo->PathName : static_cast<const TCHAR*>(TEXT("")));
				}
				else
				{
					// Keep track of any data change.
					FSkeletalMeshInfo& PrevMeshInfo = OutSkeletalMeshInfos[SkelMeshCmpIndex];
					
					bool bSameMesh = SkeletalMeshInfo->Id == PrevMeshInfo.Id; 
					bool bSameSkeleton = SkeletalMeshInfo->SkeletonId == PrevMeshInfo.SkeletonId;
					
					if (!bSameMesh)
					{
						UE_LOGFMT(LogTemp, Warning, "[{0}] - Skel Mesh Info Changed - \nPrev:{1}\nNew:{2}\nName:{3}", SkelMeshCmpIndex, PrevMeshInfo.Id, SkeletalMeshInfo->Id, SkeletalMeshObjectInfo->PathName);
						OutSkeletalMeshInfos[SkelMeshCmpIndex] = *SkeletalMeshInfo;
						
						// If mesh changes is fine but not if the skeleton changes!
						// UE_LOGFMT(LogTemp, Warning, "No SkelMeshInfo or SkelMeshObjectInfo provided. For object: {name}", SkeletalMeshObjectInfo->Name);
					}
					
					if (!bSameSkeleton)
					{
						const FObjectInfo* SkeletonObjectInfo = InGameplayProvider->FindObjectInfo(SkeletalMeshInfo->SkeletonId);
						UE_LOGFMT(LogTemp, Warning, "[{0}] - Skeleton Info Changed - \nPrev:{1}\nNew:{2}\nName:{3}", SkelMeshCmpIndex, PrevMeshInfo.SkeletonId, SkeletalMeshInfo->SkeletonId, SkeletonObjectInfo != nullptr ? SkeletonObjectInfo->PathName : static_cast<const TCHAR*>(TEXT("")));
					}
				}

				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FRewindDebuggerTrajectory::BuildTrajectories(const IGameplayProvider* InGameplayProvider, const IAnimationProvider* InAnimationProvider, double InTraceStartTime, double InTraceEndTime, const TArray<FObjectInfo>& InObjectInfos, TArray<FGameplayTrajectory>& OutTrajectories)
{
	// Clear up previous info.
	OutTrajectories.Reset();

	// Used to filter which objects we can use to build our trajectory.
	const FClassInfo* SkelMeshCmpClassInfo = InGameplayProvider->FindClassInfo(*USkeletalMeshComponent::StaticClass()->GetPathName());

	// Not a single actor with a skeletal mesh component was traced, safe to early out.
	if (SkelMeshCmpClassInfo == nullptr)
	{
		return;
	}

	// Build trajectory from pose's root transform over time.
	for (const FObjectInfo& ObjectInfo : InObjectInfos)
	{
		OutTrajectories.Push({});

		// Setup initial index range
		{
			TRange<int32> IndexRange(TRangeBound<int32>::Inclusive(0), TRangeBound<int32>::Open());

			OutTrajectories.Last().TraceInfo.Ranges.Push(IndexRange);
		}
		
		// BoneCount split tracker.
		int64 PrevSampleBoneCount = INDEX_NONE;
		
		InAnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectInfo.GetUObjectId(), [InGameplayProvider, InAnimationProvider, InTraceStartTime, InTraceEndTime, &OutTrajectories, &ObjectInfo, &PrevSampleBoneCount](const IAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bHasCurves)
		{
			InTimeline.EnumerateEvents(InTraceStartTime, InTraceEndTime, [InGameplayProvider, InAnimationProvider, &OutTrajectories, &ObjectInfo, &PrevSampleBoneCount](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InPoseMessage)
			{
				// Used to ensure trajectories always start at time being zero.
				const bool bIsFirstSample = OutTrajectories.Last().Samples.IsEmpty();
				const double FirstSampleRecordingTime = bIsFirstSample ? InPoseMessage.RecordingTime : OutTrajectories.Last().Samples[0].Time;
				
				const FVector SamplePosition = InPoseMessage.ComponentToWorld.GetTranslation();		// Root transform's position (SkelMeshCmp World Position).
				const FQuat SampleOrientation = InPoseMessage.ComponentToWorld.GetRotation();		// Root transform's position (SkelMeshCmp World Rotation).
				const double SampleTime = InPoseMessage.RecordingTime - FirstSampleRecordingTime;	// Sample timed, based of range beginning not trace beginning time.

				// Query skeletal mesh information.
				const FSkeletalMeshInfo* SkeletalMeshInfo = InAnimationProvider->FindSkeletalMeshInfo(InPoseMessage.MeshId);
				const FObjectInfo* SkeletalMeshObjectInfo = InGameplayProvider->FindObjectInfo(InPoseMessage.MeshId);
				
				if (!SkeletalMeshInfo || !SkeletalMeshObjectInfo)
				{
					OutTrajectories.Last().Reset();
					
					UE_LOGFMT(LogTemp, Warning, "No SkelMeshInfo or SkelMeshObjectInfo provided. For object: {name}", *ObjectInfo.Name);
					return TraceServices::EEventEnumerate::Stop;
				}
				
				if (InPoseMessage.NumTransforms != SkeletalMeshInfo->BoneCount)
				{
					OutTrajectories.Last().Reset();
					
					UE_LOGFMT(LogTemp, Warning, "BoneCount doesn't match. Traced bone transforms do not match the respective traced mesh asset's bones. For object: {name}", *ObjectInfo.Name);
					return TraceServices::EEventEnumerate::Stop;
				}
				
				// Keep track of instances when bone counts don't match.
				if (PrevSampleBoneCount != SkeletalMeshInfo->BoneCount)
				{
					int32 BoneCountSplitSample = OutTrajectories.Last().Samples.Num();

					// Split range by the new bone count, unless we're initializing.
					if (PrevSampleBoneCount != INDEX_NONE)
					{
						TRange<int32> LastRange = OutTrajectories.Last().TraceInfo.Ranges.Last();
						OutTrajectories.Last().TraceInfo.Ranges.Pop();
					
						OutTrajectories.Last().TraceInfo.Ranges.Append(LastRange.Split(BoneCountSplitSample));
					}
					
					// Assign the skeletal mesh info to the new range
					OutTrajectories.Last().TraceInfo.SkeletalMeshInfos.Add(*SkeletalMeshInfo);
				}
				PrevSampleBoneCount = SkeletalMeshInfo->BoneCount;

				// Extract and append animation pose for the current sample.
				{
					FTransform OutComponentToWorld;
					OutTrajectories.Last().Poses.AddDefaulted();
					InAnimationProvider->GetSkeletalMeshComponentSpacePose(InPoseMessage, *SkeletalMeshInfo, OutComponentToWorld, OutTrajectories.Last().Poses.Last());	
				}

				// Add current sample information.
				OutTrajectories.Last().Samples.Push({SampleTime, SamplePosition, SampleOrientation});

				return TraceServices::EEventEnumerate::Continue;
			});
		});

		// Always set the upper bound range.
		if (!OutTrajectories.Last().Samples.IsEmpty())
		{
			OutTrajectories.Last().TraceInfo.Ranges.Last().SetUpperBound(TRangeBound<int32>(OutTrajectories.Last().Samples.Num() - 1));
		}
		else
		{
			OutTrajectories.Last().TraceInfo.Reset();
		}
	}

	// @todo: This should report back an error if it failed to perform an action.
}

void FRewindDebuggerTrajectory::UpdateDebugInfos(const IGameplayProvider* InGameplayProvider, const TArray<FObjectInfo>& InObjectInfos, TArray<FExtensionState::FDebugInfo>& InOutDebugInfos)
{
	// We need a local copy to search for previous occurances.
	TArray<FExtensionState::FDebugInfo> PrevDebugInfos = InOutDebugInfos;
	InOutDebugInfos.Reset();
	
	for (const FObjectInfo& ObjectInfo : InObjectInfos)
	{
		const FExtensionState::FDebugInfo* MatchedDebugInfo = PrevDebugInfos.FindByPredicate([&ObjectInfo](const FExtensionState::FDebugInfo& Item)
		{
			return Item.OwnerId == ObjectInfo.GetUObjectId();
		});

		// Use cached version to keep track of stateful date (i.e. ShouldDraw checkbox)
		if (MatchedDebugInfo)
		{
			InOutDebugInfos.Push(*MatchedDebugInfo);
		}
		else
		{
			InOutDebugInfos.Push({ObjectInfo.GetUObjectId(), GetFullNameForDebugInfoOwner(InGameplayProvider, ObjectInfo.GetUObjectId()), false});
		}
	}
}

FName FRewindDebuggerTrajectory::GetFullNameForDebugInfoOwner(const IGameplayProvider* InGameplayProvider, uint64 InOwnerObjectId)
{
	const FClassInfo* ActorClassInfo = InGameplayProvider->FindClassInfo(*AActor::StaticClass()->GetPathName());

	uint64 ObjectId = InOwnerObjectId;

	TStringBuilder<128> Result;
	
	while(true)
	{
		const FObjectInfo& ObjectInfo = InGameplayProvider->GetObjectInfo(ObjectId);

		// We reached the owning actor.
		if (InGameplayProvider->IsSubClassOf(ObjectInfo.ClassId, ActorClassInfo->Id))
		{
			Result.Prepend(TEXTVIEW(" - "));
			Result.Prepend(ObjectInfo.Name);

			// Add world context
			Result.Append(TEXTVIEW(" ("));
			const FWorldInfo* WorldInfo = InGameplayProvider->FindWorldInfoFromObject(ObjectId);
			if (WorldInfo)
			{
				Result.Append(GetDebugInfoOwnerWorldPostFix(*WorldInfo).ToString());
			}
			else
			{
				Result.Append(TEXTVIEW("Unknown"));
			}
			Result.Append(TEXTVIEW(")"));
			
			return FName(Result.ToView());
		}
		// We are a component, keep traversing up the tree.
		else
		{
			if (ObjectInfo.GetOuterId().IsSet())
			{
				if (ObjectId != InOwnerObjectId)
				{
					Result.Prepend(TEXTVIEW(" - "));
				}
				Result.Prepend(ObjectInfo.Name);
				
				ObjectId = ObjectInfo.GetOuterUObjectId();
			}
			else
			{
				// Add world context
				Result.Append(TEXTVIEW(" ("));
				const FWorldInfo* WorldInfo = InGameplayProvider->FindWorldInfoFromObject(ObjectId);
				if (WorldInfo)
				{
					Result.Append(GetDebugInfoOwnerWorldPostFix(*WorldInfo).ToString());
				}
				else
				{
					Result.Append(TEXTVIEW("Unknown"));
				}
				Result.Append(TEXTVIEW(")"));
				
				return FName(Result.ToView());
			}
		}
	}
}

// copied from FGameplayTrack::GetWorldPostFix()
FText FRewindDebuggerTrajectory::GetDebugInfoOwnerWorldPostFix(const FWorldInfo& InWorldInfo)
{
	switch (InWorldInfo.Type)
	{
		default:
			return LOCTEXT("Unknown", "Unknown");
		
		case FWorldInfo::EType::None:
			return LOCTEXT("None", "None");
		
		case FWorldInfo::EType::PIE:
			{
				switch (InWorldInfo.NetMode)
				{
				case FWorldInfo::ENetMode::Client:
					return FText::Format(LOCTEXT("ClientIndexFormat", "Client {0}"), InWorldInfo.PIEInstanceId - 1);
					
				case FWorldInfo::ENetMode::DedicatedServer:
					return LOCTEXT("ServerPostfix", "Server");
					
				case FWorldInfo::ENetMode::ListenServer:
					return LOCTEXT("ListenServerPostfix", "Listen Server");
					
				case FWorldInfo::ENetMode::Standalone:
					return InWorldInfo.bIsSimulating ? LOCTEXT("SimulateInEditorPostfix", "Simulate") : LOCTEXT("PlayInEditorPostfix", "PIE");
					
				default: 
					return LOCTEXT("UnknownNetModePIEPostFix", "PIE");
				}
			}
		
		case FWorldInfo::EType::Editor:
		case FWorldInfo::EType::EditorPreview:
			return LOCTEXT("EditorPostfix", "Editor");
		
		case FWorldInfo::EType::Game:
		case FWorldInfo::EType::GamePreview:
		case FWorldInfo::EType::GameRPC:
			return LOCTEXT("GamePostfix", "Game");
		
		case FWorldInfo::EType::Inactive:
			return LOCTEXT("InactivePostfix", "Inactive");
	}
}

#undef LOCTEXT_NAMESPACE
