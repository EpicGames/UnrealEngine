// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "TrajectoryRewindDebuggerExtension.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"

#define LOCTEXT_NAMESPACE "FTrajectoryEditorModule"

class FTrajectoryToolsModule : public IModuleInterface
{
private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FRewindDebuggerTrajectory RewindDebuggerTrajectoryExtension;
};

void FTrajectoryToolsModule::StartupModule()
{
	if (GIsEditor && !IsRunningCommandlet())
	{
		RewindDebuggerTrajectoryExtension.Initialize();
		IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerTrajectoryExtension);
	}
}

void FTrajectoryToolsModule::ShutdownModule()
{
	RewindDebuggerTrajectoryExtension.Shutdown();
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerTrajectoryExtension);
}

IMPLEMENT_MODULE(FTrajectoryToolsModule, TrajectoryTools)

#undef LOCTEXT_NAMESPACE
