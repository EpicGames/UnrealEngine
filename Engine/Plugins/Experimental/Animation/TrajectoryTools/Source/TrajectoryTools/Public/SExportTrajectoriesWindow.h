// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TrajectoryLibrary.h"
#include "Widgets/SCompoundWidget.h"
#include "SEditorViewport.h"
#include "PreviewScene.h"
#include "EditorViewportClient.h"
#include "TrajectoryRewindDebuggerExtension.h" // @todo: extract nested class and forward declare
// #include "IAnimationProvider.h"

#include "SExportTrajectoriesWindow.generated.h"

class SMenuAnchor;
class UAnimSequence;
class SWindow;
class ITableRow;
class STableViewBase;
class IDetailsView;
class FSceneView;
class FCanvas;
class FViewport;
class UDebugSkelMeshComponent;

struct FSkeletalMeshInfo;

// @todo: have a UTrajectoryViewportSettings, to store config / flags.

UCLASS()
class UTrajectoryExportDetails : public UObject
{
	GENERATED_BODY()

public:
	
	/** Used to determine how the trajectory will be transformed into a animation sequence */
	UPROPERTY(EditAnywhere, Category = "Export Settings", meta=(ShowOnlyInnerProperties))
	FTrajectoryExportSettings ExportSettings;
	
	/** Number of key frames the animation sequence will have */
	UPROPERTY(VisibleAnywhere, Category = "Information")
	int NumberOfKeyFrames = 0;

	/** Play length of the sequence obtained after backing trajectory */
	UPROPERTY(VisibleAnywhere, Category = "Information")
	double PlayLength = 0;

	/** Used to create an asset for the trajectory to export its data on to. */
	UPROPERTY(EditAnywhere, Category = "Output Asset", meta=(ShowOnlyInnerProperties))
	FTrajectoryExportAssetInfo ExportAssetInfo;
	
	/** Reset all properties back to default values */
	void Reset()
	{
		const UTrajectoryExportDetails* DefaultExportDetails = GetDefault<UTrajectoryExportDetails>();
		check(DefaultExportDetails)
		
		ExportSettings.Reset();
		ExportAssetInfo.Reset();
		
		// OutputAsset = DefaultExportDetails->OutputAsset;
		PlayLength = DefaultExportDetails->PlayLength;
		NumberOfKeyFrames = DefaultExportDetails->NumberOfKeyFrames;
	}

	// virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};

/** Holds all the information needed to display a trajectory in the preview viewport */
struct FPreviewData
{
	/** Index to access all the information related to the currently selected trajectory */
	int SourceIndex = INDEX_NONE;

	struct FViewportInfo
	{
		uint64 SourceSkeletalMeshComponentId = 0;
		FSkeletalMeshInfo SourceSkeletalMeshInfo;
	} Viewport;

	/** Raw trajectories available */
	TConstArrayView<FGameplayTrajectory> SourceTrajectories;
	
	/** Preview trajectory which is affected by the export settings */
	FGameplayTrajectory OutputTrajectory;

	/** We need to hold a pointer to be able to query the export settings and accurately display a preview trajectory */
	TObjectPtr<UTrajectoryExportDetails> Details = nullptr; // @todo: this should be a tweak object ptr, tbh.

	/** Holds stateful data used to draw timeline */
	struct FTimelineInfo
	{
		bool bShouldDrawScrubber = false;
		double ScrubTime = 0;
	} Timeline;
};

/** Used to preview the trajectory to export given its settings (ExportRange,ExportFrameRage, etc). */
class SPreviewTrajectoryViewport : public SEditorViewport
{
	
public:
	
	SLATE_BEGIN_ARGS(SPreviewTrajectoryViewport) {}
		SLATE_ARGUMENT(TSharedPtr<FPreviewData>, PreviewData)
	SLATE_END_ARGS()

	SPreviewTrajectoryViewport() : PreviewScene(FPreviewScene::ConstructionValues()), PreviewData(nullptr) {};
	
	void Construct(const FArguments& InArgs);

	/** End SWidget */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	/** End SWidget */

	void ResetPreviewSkeletalMesh() const;
	void UpdatePreviewPoseFromScrubTime(double InTraceStartTime, double InTraceEndTime) const;
	
	TSharedPtr<FPreviewData> GetPreviewData() const { return PreviewData; }

protected:
	/** Begin SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	/** End SEditorViewport interface */

private:
	
	/** Scene to preview trajectory on */
	FPreviewScene PreviewScene;

	/** Used to draw the trajectory */
	TSharedPtr<FPreviewData> PreviewData;

	/** Used to display the pose of the scrub time */
	TObjectPtr<UDebugSkelMeshComponent> PreviewComponent;
};

/** Client for SPreviewTrajectoryViewport */
class FPreviewTrajectoryViewportClient : public FEditorViewportClient
{
	
public:
	
	FPreviewTrajectoryViewportClient(FPreviewScene& InPreviewScene, const TSharedRef<SPreviewTrajectoryViewport>& InPreviewTrajectoryViewport);

	/** Begin FEditorViewportClient interface */
	virtual void Tick(float DeltaTime) override;
	virtual FSceneInterface* GetScene() const override { return PreviewScene->GetScene(); }
	virtual FLinearColor GetBackgroundColor() const override { return FLinearColor(0.36f, 0.36f, 0.36f, 1); }
	virtual void SetViewMode(EViewModeIndex Index) override final { FEditorViewportClient::SetViewMode(Index); }
	virtual void Draw(const FSceneView* InView, FPrimitiveDrawInterface* InPDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& InView, FCanvas& InCanvas) override;
	/** End FEditorViewportClient interface */
};

/** Widget that configures and export a trajectory(s) into an animation asset / sequence */
class SExportTrajectoriesWindow : public SCompoundWidget, FGCObject
{
public:

	SLATE_BEGIN_ARGS(SExportTrajectoriesWindow) {}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(TArrayView<FName>, OwnerNames)
		SLATE_ARGUMENT(TArrayView<FRewindDebuggerTrajectory::FExtensionState::FDebugInfo>, DebugInfos)
		SLATE_ARGUMENT(TArrayView<FGameplayTrajectory>, Trajectories)
		SLATE_ARGUMENT(TArrayView<FSkeletalMeshInfo>, SkelMeshInfos)
	SLATE_END_ARGS()

	SExportTrajectoriesWindow() {}

	/** Usual Construct method of SWidgets. Constructs the slate UI and assigns the UI delegates. */
	void Construct(const FArguments& InArgs);

	/** Begin FGCObject */
	virtual FString GetReferencerName() const override	{ return TEXT("SExportTrajectoriesWindow"); }
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	/** End FGCObject */
	
private:

	enum class EExportValidation
	{
		MultipleTraceRanges,
		NoTrajectorySelected,
		MissingAssetInfo,
		ExportPathIsInvalid,
		AssetAlreadyExists,
		Valid,
	};

	// CALLBACKS
	
	TSharedRef<SWidget> OnGetTrajectoryPickerMenuContent();
	void OnTrajectoryPickerMenuOpened(bool bIsOpen);
	void OnFinishedChangingExportSettingsSelectionProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	// HELPER

	FText GetWarningMessage() const;
	EExportValidation GetExportValidation() const;
	void UpdateAssetInfo(FSoftObjectPath&  OutSkeletonPath, FSoftObjectPath & OutSkeletalMeshPath) const;
	void UpdateViewportAfterTrajectorySelection();
	
	/** Window owning this widget */
	TWeakPtr<SWindow> WidgetWindow;

	// INPUT DATA (Constant)

	struct ImmutableState
	{
		// TArrayView<FGameplayTrajectory> Trajectories;
		TArrayView<FRewindDebuggerTrajectory::FExtensionState::FDebugInfo> DebugInfos;
		TArrayView<FSkeletalMeshInfo> SkeletalMeshInfos;
		// TArrayView<FName> OwnerNames;
	} ImmutableState;
	
	/** Trajectories that are available for export */
	TArray<FGameplayTrajectory> Trajectories;

	/** Names of the trajectories' associated objects */
	TArray<FName> TrajectoryOwnerNames;

	// SELECTION DATA

	/** Index of trajectory to be exported */
	int SelectedTrajectoryIndex = INDEX_NONE;
	
	/** Used to determine how the trajectory should be exported / baked out */
	TObjectPtr<UTrajectoryExportDetails> ExportDetails;
	
	// PREVIEW DATA
	
	TSharedPtr<FPreviewData> PreviewData;
	
	// WIDGETS
	
	TSharedPtr<IDetailsView> ExportDetailsView;
	TSharedPtr<SPreviewTrajectoryViewport> Viewport;
	TSharedPtr<SMenuAnchor> TrajectoryPickerComboButton;
	TSharedPtr<SWidget> ExportButton;
	TSharedPtr<SWidget> ViewportTimeline;
};
