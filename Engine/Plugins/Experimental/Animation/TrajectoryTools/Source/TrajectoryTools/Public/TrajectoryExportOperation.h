// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/FrameRate.h"
#include "ScopedTransaction.h"
#include "TrajectoryExportOperation.generated.h"

#define UE_API TRAJECTORYTOOLS_API

struct FGameplayTrajectory;
class UAnimSequence;

USTRUCT()
struct FTrajectoryExportSettings
{
	GENERATED_BODY()
	
	UE_API FTrajectoryExportSettings();
	
	/** Frame rate for the exported asset */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FFrameRate FrameRate;
	
	/** Range of the trajectory to export. Note that the entire trajectory will be exported if the range is empty (Min == Max). */
	UPROPERTY(EditAnywhere, meta=(Units="s"), Category = "Settings")
	FFloatInterval Range;

	/** Flag to determine if all trajectory samples should be relative to the frame given by ForceOriginAtTime*/
	UPROPERTY(EditAnywhere, meta=(DisplayName="Force Origin"), Category = "Settings")
	bool bShouldForceOrigin = false;
	
	/** Specific which trajectory sample's position will be position at the origin given by the desired time. Note that if time for origin is less than 0 no origin will be enforced. */
	UPROPERTY(EditAnywhere, meta=(EditCondition="bShouldForceOrigin", EditConditionHides, Units="s"), Category = "Settings")
	double OriginTime = 0;

	/**
	 * Any files with the same name will be overwritten instead of creating a new file with a numeric suffix.
	 * This is useful when iterating on a batch process.
	*/
	UPROPERTY(EditAnywhere, meta=(DisplayName="Overwrite Existing Files"), Category = "Settings")
	bool bShouldOverwriteExistingFiles = false;

	/** Will not produce keys on bones that are not animated, reducing size on disk of the resulting files. */
	UPROPERTY(EditAnywhere, meta=(DisplayName="Export Only Animated Bones"), Category = "Settings")
	bool bShouldExportOnlyAnimatedBones = true;
	
	/** All export settings are valid and trajectory can be generated without issues from them */
	UE_API bool IsValid() const;

	/** Reset trajectory to default values */
	UE_API void Reset();
};

/** All information needed to create an asset for a trajectory */
USTRUCT()
struct FTrajectoryExportAssetInfo
{
	GENERATED_BODY()
	
	/** Name of asset to be created during export. Defaulted to the name of the selected trajectory in UI. */
	UPROPERTY(EditAnywhere, Category = "Asset")
	FString AssetName;
	
	/** Directory path where to export asset to */
	UPROPERTY(EditAnywhere, Category = "Asset", meta=(ContentDir))
	FDirectoryPath FolderPath;

	/** Path to skeleton to be used and assigned to the exported asset. */
	UPROPERTY(VisibleAnywhere, Category = "Information")
	FSoftObjectPath Skeleton;

	/** Path to skeletal mesh to be used and assigned to the exported asset. */
	UPROPERTY(VisibleAnywhere, Category = "Information")
	FSoftObjectPath SkeletalMesh;

	UE_API bool IsExportPathValid() const;
	UE_API bool DoesAssetExistAtPath() const;
	UE_API bool CanCreateAsset() const;
	
	UE_API void Reset();

	UE_API bool IsValid() const;
};

/** Data needed to run an "export" operation on a trajectory. */
struct FTrajectoryExportContext
{
	/** Used when transforming trajectory data before saving/exporting it to an asset. */
	FTrajectoryExportSettings Settings;
	
	/** Used when creating the asset to hold the trajectory data. */
	FTrajectoryExportAssetInfo AssetInfo;

	/** Raw trajectory data that will be export after operation is complete. */
	FGameplayTrajectory* Data;

	/** Name of the object that we sourced the trajectory data from */
	FString SourceObjectName;
	
	UE_API void Reset();

	UE_API bool IsValid() const;
};

/** Encapsulate ability to export a trajectory to the specified asset. */
UCLASS(MinimalAPI)
class UTrajectoryExportOperation : public UObject
{
	GENERATED_BODY()
	
public:

	// @todo: After having a general trajectory type, expose this to blueprint to allow for editor scripts, bp, etc to use this.
	static UE_API void ExportTrajectory(FGameplayTrajectory* InTrajectory, const FTrajectoryExportSettings& InSettings, const FTrajectoryExportAssetInfo& InAssetInfo, const FString& InSourceObjectName);
	
	/** Actually run the process to export the trajectory for the given context. */
	UE_API void Run(const FTrajectoryExportContext& Context);
	
private:

	/** Clear any generated/saved info. */
	UE_API void Reset();

	/** Create assets to export the trajectory data to. */
	UE_API void GenerateAssets(const FTrajectoryExportContext& Context, FScopedSlowTask& Progress);

	/** Output trajectory data to their respective assets. */
	UE_API void ExportDataToAssets(const FTrajectoryExportContext& Context, FScopedSlowTask& Progress) const;

	/** Output notifications of results. */
	UE_API void NotifyUserOfResults(const FTrajectoryExportContext& Context, FScopedSlowTask& Progress) const;
	
	/** If user cancelled half way, cleanup all created asset(s). */
	UE_API void CleanupIfCancelled(const FScopedSlowTask& Progress) const;

	TWeakObjectPtr<UAnimSequence> GeneratedAsset = nullptr;
	TWeakObjectPtr<UAnimSequence> AssetToProcess = nullptr;

	TUniquePtr<FScopedTransaction> ActiveTransaction = nullptr;
};

#undef UE_API
