// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "TrajectoryExportOperation.h"
#include "IAnimationProvider.h"
#include "TrajectoryLibrary.generated.h"

#define UE_API TRAJECTORYTOOLS_API

class UAnimSequence;
struct FTrajectoryExportSettings;

// @todo: Temp. This will be swapped for general purpose trajectory struct + poses array.
USTRUCT()
struct FGameplayTrajectory
{
	GENERATED_BODY();
	
	struct FSample
	{
		double Time = 0;
		FVector Position = FVector::ZeroVector;
		FQuat Orientation = FQuat::Identity;
	};
	
	TArray<FSample> Samples;
	TArray<TArray<FTransform>> Poses;
	
	struct FTraceRangedBuffers
	{
		TArray<TRange<int32>> Ranges;
		TArray<FSkeletalMeshInfo> SkeletalMeshInfos;

		int Num() const
		{
			ensure(Ranges.Num() == SkeletalMeshInfos.Num());
			return Ranges.Num();
		}
		
		bool IsEmpty() const
		{
			return ensure(Ranges.IsEmpty() == SkeletalMeshInfos.IsEmpty()) && Ranges.IsEmpty();
		}

		void Reset(int NewSize = 0)
		{
			Ranges.Reset(NewSize);
			SkeletalMeshInfos.Reset(NewSize);
		}
		
	} TraceInfo;
	
	void Reset()
	{
		Samples.Reset();
		Poses.Reset();
		TraceInfo.Ranges.Reset();
		TraceInfo.SkeletalMeshInfos.Reset();
	}

	bool IsValid() const
	{
		return Samples.Num() == Poses.Num();
	}
};

struct FTrajectoryToolsLibrary
{
	struct FRangeOverlapTestResult
	{
		bool bOverlaps = false;
		TArray<int> Ranges;
	};
	
	static UE_API void GetRangeOverlaps(const FGameplayTrajectory& InTrajectory, const TRange<int32>& InSampleRange, FRangeOverlapTestResult& InOutOverlapResult);
	
	static UE_API void GetSampleIndicesForMatchedSampleTime(const FGameplayTrajectory& InTrajectory, double InRequestedSampleTime, TPair<int32, int32>& OutMatchedSampleIndicesRange);

	static UE_API bool GetTransformAtTimeInTrajectory(const FGameplayTrajectory& InTrajectory, double InRequestedTime, FTransform& OuTransform);

	static UE_API bool GetPoseAtTimeInTrajectory(const FGameplayTrajectory& InTrajectory, double InRequestedTime, TArray<FTransform>& OutCSPose);
	
	static UE_API void TransformTrajectoryToMatchFrameRate(const FGameplayTrajectory& InTrajectory, FFrameRate InFrameRate, FGameplayTrajectory& OutFrameMatchedTrajectory);

	static UE_API void TransformTrajectoryToMatchExportSettings(const FGameplayTrajectory& InTrajectory, const FTrajectoryExportSettings& InExportSettings, FGameplayTrajectory& OutTransformedTrajectory);
};

#undef UE_API
