// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerExtension.h"
#include "TrajectoryLibrary.h"

class APlayerController;
class UCanvas;
class UToolMenu;
struct FToolMenuSection;
class SWindow;

/**
 * A Rewind Debugger extension that allows user to visualize trajectories and export them from the current trace session.
 */
class FRewindDebuggerTrajectory : public IRewindDebuggerExtension
{
public:
	
	FRewindDebuggerTrajectory();
	virtual ~FRewindDebuggerTrajectory();

	void Initialize();
	void Shutdown();
	void Reset();

	/** Begin IRewindDebuggerExtension implementation */
	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual void AnalysisSessionOpened(IRewindDebugger* RewindDebugger) override;
	virtual void AnalysisSessionClosed(IRewindDebugger* RewindDebugger) override;
	virtual void Clear(IRewindDebugger* RewindDebugger) override;
	virtual FString GetName() { return TEXT("RewindDebuggerTrajectory"); }
	/* End IRewindDebuggerExtension implementation */
	
private:

	void UpdateState(IRewindDebugger* RewindDebugger);
	
	// UI
	
	void GenerateMenu();
	void MakeTrajectoriesMenu(FToolMenuSection& InSection);;
	void MakeToggleDebugDrawMenu(UToolMenu* InMenu);
	
	// Debug Draw
	
	void EnsureDebugDrawDelegateExists();
	void DebugDraw(UCanvas* InCanvas, APlayerController* InController);

	/** Holds information from each extension's update */
	struct FExtensionState
	{
		/** Information for UI / debug drawing */
		struct FDebugInfo
		{
			uint64 OwnerId = 0;
			FName Name = NAME_None;
			bool bShouldDraw = false;
		};

		/** Information to extract a trajectory/clip from a trace file */
		struct FExtract
		{
			double TraceStarTime = 0;
			double TraceEndTime = 0;
		} Extract;
		
		TArray<FGameplayTrajectory> Trajectories;	// Trajectories extracted from trace session
		TArray<FObjectInfo> ObjectInfos;			// Associated object's information for the extracted trajectories
		TArray<FSkeletalMeshInfo> SkelMeshInfos;	// Associated object's skeletal mesh info for the extracted trajectories.	
		TArray<FDebugInfo> DebugInfos;				// Associated debug information for extracted trajectories
		
		/** Reset extension's state */
		void Reset()
		{
			Extract.TraceEndTime = 0;
			Extract.TraceStarTime = 0;
			
			Trajectories.Reset();
			ObjectInfos.Reset();
			SkelMeshInfos.Reset();
			DebugInfos.Reset();
		}
	} State;
	
	TArray<FColor> DebugDrawColors;
	FDelegateHandle DebugDrawDelegateHandle;
	TWeakObjectPtr<UWorld> WorldToVisualize;
	TSharedPtr<SWindow> BakeOutWindow;
	
	void ShowBakeOutTrajectoryWindow();

	static void BuildTrajectoryOwnersList(const IRewindDebugger* InRewindDebugger, const IGameplayProvider* InGameplayProvider, double InTraceStartTime, double InTraceEndTime, TArray<FObjectInfo> & OutObjectInfos);
	static void BuildTrajectorySkeletalMeshInfoList(const IGameplayProvider* InGameplayProvider, const IAnimationProvider* InAnimationProvider, double InTraceStartTime, double InTraceEndTime, const TArray<FObjectInfo>& InSkeletalMeshComponentObjectInfos, TArray<FSkeletalMeshInfo>& OutSkeletalMeshInfos);
	static void BuildTrajectories(const IGameplayProvider* InGameplayProvider, const IAnimationProvider* InAnimationProvider, double InTraceStartTime, double InTraceEndTime, const TArray<FObjectInfo>& InObjectInfos, TArray<FGameplayTrajectory>& OutTrajectories);
	static void UpdateDebugInfos(const IGameplayProvider* InGameplayProvider, const TArray<FObjectInfo>& InObjectInfos, TArray<FExtensionState::FDebugInfo>& InOutDebugInfos);
	static FName GetFullNameForDebugInfoOwner(const IGameplayProvider* InGameplayProvider, uint64 InOwnerObjectId);
	static FText GetDebugInfoOwnerWorldPostFix(const FWorldInfo& InWorldInfo);
	
	friend class SExportTrajectoriesWindow;
};
