// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TrajectoryTools : ModuleRules
{
	public TrajectoryTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				
				// Trace-related dependencies
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"GameplayInsights",
				"RewindDebugger",
				
				// UI 
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"RewindDebuggerInterface",
				"UnrealEd",
				"InputCore",
				"ToolMenus",
				"ToolWidgets",
				"EditorWidgets", 
				
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
